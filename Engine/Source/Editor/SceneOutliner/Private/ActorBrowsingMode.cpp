// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorBrowsingMode.h"

#include "ActorBrowsingModeCommands.h"
#include "Engine/Blueprint.h"
#include "SceneOutlinerFilters.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerMenuContext.h"
#include "ActorHierarchy.h"
#include "SceneOutlinerDelegates.h"
#include "Editor.h"
#include "Editor/GroupActor.h"
#include "UnrealEdGlobals.h"
#include "ToolMenus.h"
#include "Engine/Selection.h"
#include "Engine/World.h"
#include "Editor/UnrealEdEngine.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/FileManager.h"
#include "SceneOutlinerDragDrop.h"
#include "EditorActorFolders.h"
#include "EditorFolderUtils.h"
#include "ActorEditorUtils.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "Logging/MessageLog.h"
#include "SSocketChooser.h"
#include "ActorFolderPickingMode.h"
#include "ActorTreeItem.h"
#include "LevelTreeItem.h"
#include "ActorFolderTreeItem.h"
#include "WorldTreeItem.h"
#include "ComponentTreeItem.h"
#include "ActorBrowsingModeSettings.h"
#include "ActorDescTreeItem.h"
#include "ScopedTransaction.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "EditorLevelUtils.h"
#include "EditorViewportCommands.h"
#include "Misc/ScopedSlowTask.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Framework/Commands/GenericCommands.h"

DEFINE_LOG_CATEGORY_STATIC(LogActorBrowser, Log, All);

#define LOCTEXT_NAMESPACE "SceneOutliner_ActorBrowsingMode"

using FActorFilter = TSceneOutlinerPredicateFilter<FActorTreeItem>;
using FActorDescFilter = TSceneOutlinerPredicateFilter<FActorDescTreeItem>;
using FFolderFilter = TSceneOutlinerPredicateFilter<FFolderTreeItem>;

FActorBrowsingMode::FActorBrowsingMode(SSceneOutliner* InSceneOutliner, TWeakObjectPtr<UWorld> InSpecifiedWorldToDisplay)
	: FActorModeInteractive(FActorModeParams(InSceneOutliner, InSpecifiedWorldToDisplay,  /* bHideComponents */ true, /* bHideLevelInstanceHierarchy */ false, /* bHideUnloadedActors */ false, /* bHideEmptyFolders */ false))
	, FilteredActorCount(0)
{
	WorldPartitionEditorModule = FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");

	// Capture selection changes of bones from mesh selection in fracture tools
	FSceneOutlinerDelegates::Get().OnComponentsUpdated.AddRaw(this, &FActorBrowsingMode::OnComponentsUpdated);

	GEngine->OnLevelActorDeleted().AddRaw(this, &FActorBrowsingMode::OnLevelActorDeleted);

	GEditor->OnSelectUnloadedActorsEvent().AddRaw(this, &FActorBrowsingMode::OnSelectUnloadedActors);

	UActorEditorContextSubsystem::Get()->OnActorEditorContextSubsystemChanged().AddRaw(this, &FActorBrowsingMode::OnActorEditorContextSubsystemChanged);

	FEditorDelegates::OnEditCutActorsBegin.AddRaw(this, &FActorBrowsingMode::OnEditCutActorsBegin);
	FEditorDelegates::OnEditCutActorsEnd.AddRaw(this, &FActorBrowsingMode::OnEditCutActorsEnd);
	FEditorDelegates::OnEditCopyActorsBegin.AddRaw(this, &FActorBrowsingMode::OnEditCopyActorsBegin);
	FEditorDelegates::OnEditCopyActorsEnd.AddRaw(this, &FActorBrowsingMode::OnEditCopyActorsEnd);
	FEditorDelegates::OnEditPasteActorsBegin.AddRaw(this, &FActorBrowsingMode::OnEditPasteActorsBegin);
	FEditorDelegates::OnEditPasteActorsEnd.AddRaw(this, &FActorBrowsingMode::OnEditPasteActorsEnd);
	FEditorDelegates::OnDuplicateActorsBegin.AddRaw(this, &FActorBrowsingMode::OnDuplicateActorsBegin);
	FEditorDelegates::OnDuplicateActorsEnd.AddRaw(this, &FActorBrowsingMode::OnDuplicateActorsEnd);
	FEditorDelegates::OnDeleteActorsBegin.AddRaw(this, &FActorBrowsingMode::OnDeleteActorsBegin);
	FEditorDelegates::OnDeleteActorsEnd.AddRaw(this, &FActorBrowsingMode::OnDeleteActorsEnd);

	UActorBrowserConfig::Initialize();
	UActorBrowserConfig::Get()->LoadEditorConfig();

	// Get a MutableConfig here to force create a config for the current outliner if it doesn't exist
	const FActorBrowsingModeConfig* SavedSettings = GetMutableConfig();

	// Create a local struct to use the default values if this outliner doesn't want to save configs
	FActorBrowsingModeConfig LocalSettings;

	// If this outliner doesn't want to save config (OutlinerIdentifier is empty, use the defaults)
	if (SavedSettings)
	{
		LocalSettings = *SavedSettings;
	}

	bHideLevelInstanceHierarchy = LocalSettings.bHideLevelInstanceHierarchy;
	InSceneOutliner->SetShowTransient(!LocalSettings.bHideTemporaryActors);
	bShouldUpdateContentWhileInPIEFocused = LocalSettings.bShouldUpdateContentWhileInPIEFocused;

	// Get the OutlinerModule to register FilterInfos with the FilterInfoMap
	FSceneOutlinerFilterInfo ShowOnlySelectedActorsInfo(LOCTEXT("ToggleShowOnlySelected", "Only Selected"), LOCTEXT("ToggleShowOnlySelectedToolTip", "When enabled, only displays actors that are currently selected."), LocalSettings.bShowOnlySelectedActors, FCreateSceneOutlinerFilter::CreateStatic(&FActorBrowsingMode::CreateShowOnlySelectedActorsFilter));
	ShowOnlySelectedActorsInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FActorBrowsingModeConfig* Settings = GetMutableConfig();
			if(Settings)
			{
				Settings->bShowOnlySelectedActors = bIsActive;
				SaveConfig();
			}
		});
	FilterInfoMap.Add(TEXT("ShowOnlySelectedActors"), ShowOnlySelectedActorsInfo);
		
	FSceneOutlinerFilterInfo OnlyCurrentLevelInfo(LOCTEXT("ToggleShowOnlyCurrentLevel", "Only in Current Level"), LOCTEXT("ToggleShowOnlyCurrentLevelToolTip", "When enabled, only shows Actors that are in the Current Level."), LocalSettings.bShowOnlyActorsInCurrentLevel, FCreateSceneOutlinerFilter::CreateStatic(&FActorBrowsingMode::CreateIsInCurrentLevelFilter));
	OnlyCurrentLevelInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FActorBrowsingModeConfig* Settings = GetMutableConfig();
			if(Settings)
			{
				Settings->bShowOnlyActorsInCurrentLevel = bIsActive;
				SaveConfig();
			}
		});
	FilterInfoMap.Add(TEXT("ShowOnlyCurrentLevel"), OnlyCurrentLevelInfo);

	FSceneOutlinerFilterInfo OnlyCurrentDataLayersInfo(LOCTEXT("ToggleShowOnlyCurrentDataLayers", "Only in any Current Data Layers"), LOCTEXT("ToggleShowOnlyCurrentDataLayersToolTip", "When enabled, only shows Actors that are in any Current Data Layers."), LocalSettings.bShowOnlyActorsInCurrentDataLayers, FCreateSceneOutlinerFilter::CreateStatic(&FActorBrowsingMode::CreateIsInCurrentDataLayersFilter));
	OnlyCurrentDataLayersInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FActorBrowsingModeConfig* Settings = GetMutableConfig();
			if (Settings)
			{
				Settings->bShowOnlyActorsInCurrentDataLayers = bIsActive;
				SaveConfig();
			}
		});
	FilterInfoMap.Add(TEXT("ShowOnlyCurrentDataLayers"), OnlyCurrentDataLayersInfo);

	// Add a filter for unloaded actors to properly reflect the bShowOnlyActorsInCurrentDataLayers flag.
	SceneOutliner->AddFilter(MakeShared<FActorDescFilter>(FActorDescTreeItem::FFilterPredicate::CreateLambda([this](const FWorldPartitionActorDescInstance* ActorDescInstance)
		{
			FActorBrowsingModeConfig* Settings = GetMutableConfig();
			if (Settings && Settings->bShowOnlyActorsInCurrentDataLayers)
			{
				const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(RepresentingWorld.Get());
				if (!DataLayerManager || DataLayerManager->GetActorEditorContextDataLayers().IsEmpty())
				{
					return true;
				}
				
				for(const UDataLayerInstance* const DataLayerInstance : DataLayerManager->GetDataLayerInstances(ActorDescInstance->GetDataLayerInstanceNames().ToArray()))
				{
					if (DataLayerInstance->IsInActorEditorContext())
					{
						return true;
					}
				}
				return false;
			}
			return true;
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	FSceneOutlinerFilterInfo OnlyCurrentContentBundleInfo(LOCTEXT("ToggleShowOnlyCurrentContentBundle", "Only in Current Content Bundle"), LOCTEXT("ToggleShowOnlyCurrentContentBundleToolTip", "When enabled, only shows Actors that are in the Current Content Bundle."), LocalSettings.bShowOnlyActorsInCurrentContentBundle, FCreateSceneOutlinerFilter::CreateRaw(this, &FActorBrowsingMode::CreateIsInCurrentContentBundleFilter));
	OnlyCurrentContentBundleInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FActorBrowsingModeConfig* Settings = GetMutableConfig();
			if (Settings)
			{
				Settings->bShowOnlyActorsInCurrentContentBundle = bIsActive;
				SaveConfig();
			}
		});
	FilterInfoMap.Add(TEXT("ShowOnlyCurrentContentBundle"), OnlyCurrentContentBundleInfo);

	// Add a filter for unloaded actors to properly reflect the bShowOnlyActorsInCurrentContentBundle flag.
	SceneOutliner->AddFilter(MakeShared<FActorDescFilter>(FActorDescTreeItem::FFilterPredicate::CreateLambda([this](const FWorldPartitionActorDescInstance* ActorDescInstance)
		{
			FActorBrowsingModeConfig* Settings = GetMutableConfig();
			if (Settings && Settings->bShowOnlyActorsInCurrentContentBundle)
			{
				if (!WorldPartitionEditorModule || !WorldPartitionEditorModule->IsEditingContentBundle())
				{
					return true;
				}

				return ActorDescInstance->GetContentBundleGuid().IsValid() && WorldPartitionEditorModule->IsEditingContentBundle(ActorDescInstance->GetContentBundleGuid());
			}
			return true;
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	bHideComponents = LocalSettings.bHideActorComponents;
	FSceneOutlinerFilterInfo HideComponentsInfo(LOCTEXT("ToggleHideActorComponents", "Hide Actor Components"), LOCTEXT("ToggleHideActorComponentsToolTip", "When enabled, hides components belonging to actors."), LocalSettings.bHideActorComponents, FCreateSceneOutlinerFilter::CreateStatic(&FActorBrowsingMode::CreateHideComponentsFilter));
	HideComponentsInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FActorBrowsingModeConfig* Settings = GetMutableConfig();
			if(Settings)
			{
				Settings->bHideActorComponents = bHideComponents = bIsActive;
				SaveConfig();
			}
			if (auto ActorHierarchy = StaticCast<FActorHierarchy*>(Hierarchy.Get()))
			{
				ActorHierarchy->SetShowingComponents(!bIsActive);
			}
		});

	FilterInfoMap.Add(TEXT("HideComponentsFilter"), HideComponentsInfo);

	bHideUnloadedActors = LocalSettings.bHideUnloadedActors;
	FSceneOutlinerFilterInfo HideUnloadedActorsInfo(LOCTEXT("ToggleHideUnloadedActors", "Hide Unloaded Actors"), LOCTEXT("ToggleHideUnloadedActorsToolTip", "When enabled, hides all unloaded world partition actors."), LocalSettings.bHideUnloadedActors, FCreateSceneOutlinerFilter::CreateStatic(&FActorBrowsingMode::CreateHideUnloadedActorsFilter));
	HideUnloadedActorsInfo.OnToggle().AddLambda([this] (bool bIsActive)
		{
			FActorBrowsingModeConfig* Settings = GetMutableConfig();
			if(Settings)
			{
				Settings->bHideUnloadedActors = bHideUnloadedActors = bIsActive;
				SaveConfig();
			}

			if (auto ActorHierarchy = StaticCast<FActorHierarchy*>(Hierarchy.Get()))
			{
				ActorHierarchy->SetShowingUnloadedActors(!bIsActive);
			}
		});
	FilterInfoMap.Add(TEXT("HideUnloadedActorsFilter"), HideUnloadedActorsInfo);

	bHideEmptyFolders = LocalSettings.bHideEmptyFolders;
	FSceneOutlinerFilterInfo HideEmptyFoldersInfo(LOCTEXT("ToggleHideEmptyFolders", "Hide Empty Folders"), LOCTEXT("ToggleHideEmptyFoldersToolTip", "When enabled, hides all empty folders."), LocalSettings.bHideEmptyFolders, FCreateSceneOutlinerFilter::CreateStatic(&FActorBrowsingMode::CreateHideEmptyFoldersFilter));
	HideEmptyFoldersInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FActorBrowsingModeConfig* Settings = GetMutableConfig();
			if (Settings)
			{
				Settings->bHideEmptyFolders = bHideEmptyFolders = bIsActive;
				SaveConfig();
			}

			if (auto ActorHierarchy = StaticCast<FActorHierarchy*>(Hierarchy.Get()))
			{
				ActorHierarchy->SetShowingEmptyFolders(!bIsActive);
			}
		});
	FilterInfoMap.Add(TEXT("HideEmptyFoldersFilter"), HideEmptyFoldersInfo);

	// Add a filter which sets the interactive mode of LevelInstance items and their children
	SceneOutliner->AddInteractiveFilter(MakeShared<FActorFilter>(FActorTreeItem::FFilterPredicate::CreateLambda([this](const AActor* Actor)
		{
			if (Actor->SupportsSubRootSelection())
			{
				return true;
			}

			if (!bHideLevelInstanceHierarchy)
			{
				if (const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>())
				{
					// if actor has a valid parent and the parent is not being edited,
					// then the actor should not be selectable.
					if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
					{
						if (!LevelInstanceSubsystem->IsEditingLevelInstance(ParentLevelInstance) && !LevelInstanceSubsystem->IsEditingLevelInstancePropertyOverrides(ParentLevelInstance))
						{
							return false;
						}
					}
				}
			}
			return true;
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	bAlwaysFrameSelection = LocalSettings.bAlwaysFrameSelection;
	bCollapseOutlinerTreeOnNewSelection = LocalSettings.bCollapseOutlinerTreeOnNewSelection;
	
	Rebuild();
}

FActorBrowsingMode::~FActorBrowsingMode()
{
	if (RepresentingWorld.IsValid())
	{
		if (UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			WorldPartition->OnActorDescInstanceRemovedEvent.RemoveAll(this);
		}
	}
	FSceneOutlinerDelegates::Get().OnComponentsUpdated.RemoveAll(this);

	GEngine->OnLevelActorDeleted().RemoveAll(this);

	GEditor->OnSelectUnloadedActorsEvent().RemoveAll(this);

	if (UActorEditorContextSubsystem* ActorEditorContextSubsystem = UActorEditorContextSubsystem::Get())
	{
		ActorEditorContextSubsystem->OnActorEditorContextSubsystemChanged().RemoveAll(this);
	}

	FEditorDelegates::OnEditCutActorsBegin.RemoveAll(this);
	FEditorDelegates::OnEditCutActorsEnd.RemoveAll(this);
	FEditorDelegates::OnEditCopyActorsBegin.RemoveAll(this);
	FEditorDelegates::OnEditCopyActorsEnd.RemoveAll(this);
	FEditorDelegates::OnEditPasteActorsBegin.RemoveAll(this);
	FEditorDelegates::OnEditPasteActorsEnd.RemoveAll(this);
	FEditorDelegates::OnDuplicateActorsBegin.RemoveAll(this);
	FEditorDelegates::OnDuplicateActorsEnd.RemoveAll(this);
	FEditorDelegates::OnDeleteActorsBegin.RemoveAll(this);
	FEditorDelegates::OnDeleteActorsEnd.RemoveAll(this);
}

void FActorBrowsingMode::Rebuild()
{
	// If we used to be representing a wp world, unbind delegates before rebuilding begins
	if (RepresentingWorld.IsValid())
	{
		if (UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			WorldPartition->OnActorDescInstanceRemovedEvent.RemoveAll(this);
		}
	}

	FActorMode::Rebuild();

	FilteredActorCount = 0;
	FilteredUnloadedActorCount = 0;
	ApplicableUnloadedActors.Empty();
	ApplicableActors.Empty();

	bRepresentingWorldGameWorld = RepresentingWorld.IsValid() && RepresentingWorld->IsGameWorld();
	bRepresentingWorldPartitionedWorld = RepresentingWorld.IsValid() && RepresentingWorld->IsPartitionedWorld();

	if (bRepresentingWorldPartitionedWorld)
	{
		UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition();
		WorldPartition->OnActorDescInstanceRemovedEvent.AddRaw(this, &FActorBrowsingMode::OnActorDescInstanceRemoved);
	}
}

FText FActorBrowsingMode::GetStatusText() const
{
	if (!RepresentingWorld.IsValid())
	{
		return FText();
	}

	// The number of actors in the outliner before applying the text filter
	const int32 TotalActorCount = ApplicableActors.Num() + ApplicableUnloadedActors.Num();
	const int32 SelectedActorCount = SceneOutliner->GetSelection().Num<FActorTreeItem, FActorDescTreeItem>();

	if (!SceneOutliner->IsTextFilterActive())
	{
		if (SelectedActorCount == 0)
		{
			if (bRepresentingWorldPartitionedWorld)
			{
				return FText::Format(LOCTEXT("ShowingAllLoadedActorsFmt", "{0} actors ({1} loaded)"), FText::AsNumber(FilteredActorCount), FText::AsNumber(FilteredActorCount - FilteredUnloadedActorCount));
			}
			else
			{
				return FText::Format(LOCTEXT("ShowingAllActorsFmt", "{0} actors"), FText::AsNumber(FilteredActorCount));
			}
		}
		else
		{
			return FText::Format(LOCTEXT("ShowingAllActorsSelectedFmt", "{0} actors ({1} selected)"), FText::AsNumber(FilteredActorCount), FText::AsNumber(SelectedActorCount));
		}
	}
	else if (SceneOutliner->IsTextFilterActive() && FilteredActorCount == 0)
	{
		return FText::Format(LOCTEXT("ShowingNoActorsFmt", "No matching actors ({0} total)"), FText::AsNumber(TotalActorCount));
	}
	else if (SelectedActorCount != 0)
	{
		return FText::Format(LOCTEXT("ShowingOnlySomeActorsSelectedFmt", "Showing {0} of {1} actors ({2} selected)"), FText::AsNumber(FilteredActorCount), FText::AsNumber(TotalActorCount), FText::AsNumber(SelectedActorCount));
	}
	else
	{
		return FText::Format(LOCTEXT("ShowingOnlySomeActorsFmt", "Showing {0} of {1} actors"), FText::AsNumber(FilteredActorCount), FText::AsNumber(TotalActorCount));
	}
}

FSlateColor FActorBrowsingMode::GetStatusTextColor() const
{
	if (!SceneOutliner->IsTextFilterActive())
	{
		return FSlateColor::UseForeground();
	}
	else if (FilteredActorCount == 0)
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	else
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
	}
}

void FActorBrowsingMode::OnActorEditorContextSubsystemChanged()
{
	// For performance reasons avoid doing full refresh if we don't have an active filter relying on the current Actor Editor Context
	if (const FActorBrowsingModeConfig* Settings = GetConstConfig(); Settings && (Settings->bShowOnlyActorsInCurrentDataLayers || Settings->bShowOnlyActorsInCurrentContentBundle))
	{
		SceneOutliner->FullRefresh();
	}
}

void FActorBrowsingMode::OnToggleAlwaysFrameSelection()
{
	bAlwaysFrameSelection = !bAlwaysFrameSelection;

	FActorBrowsingModeConfig* Settings = GetMutableConfig();
	if(Settings)
	{
		Settings->bAlwaysFrameSelection = bAlwaysFrameSelection;
		SaveConfig();
	}
}

bool FActorBrowsingMode::ShouldAlwaysFrameSelection() const
{
	return bAlwaysFrameSelection;
}

void FActorBrowsingMode::OnToggleHideTemporaryActors()
{
	FActorBrowsingModeConfig* Settings = GetMutableConfig();
	if (Settings)
	{
		Settings->bHideTemporaryActors = !Settings->bHideTemporaryActors;
		SaveConfig();

		SceneOutliner->SetShowTransient(!Settings->bHideTemporaryActors);
		
		SceneOutliner->FullRefresh();
	}
}

bool FActorBrowsingMode::ShouldHideTemporaryActors() const
{
	const FActorBrowsingModeConfig* Settings = GetConstConfig();
	if (Settings)
	{
		return Settings->bHideTemporaryActors;
	}

	return false;
}

void FActorBrowsingMode::OnToggleHideLevelInstanceHierarchy()
{
	bHideLevelInstanceHierarchy = !bHideLevelInstanceHierarchy;

	FActorBrowsingModeConfig* Settings = GetMutableConfig();
	if (Settings)
	{
		Settings->bHideLevelInstanceHierarchy = bHideLevelInstanceHierarchy;

		SaveConfig();
	}

	if (auto ActorHierarchy = StaticCast<FActorHierarchy*>(Hierarchy.Get()))
	{
		ActorHierarchy->SetShowingLevelInstances(!bHideLevelInstanceHierarchy);
	}

	SceneOutliner->FullRefresh();
}

bool FActorBrowsingMode::ShouldHideLevelInstanceHierarchy() const
{
	const FActorBrowsingModeConfig* Settings = GetConstConfig();
	if (Settings)
	{
		return Settings->bHideLevelInstanceHierarchy;
	}

	return false;
}

void FActorBrowsingMode::InitializeViewMenuExtender(TSharedPtr<FExtender> Extender)
{
	FActorModeInteractive::InitializeViewMenuExtender(Extender);
	
	Extender->AddMenuExtension(SceneOutliner::ExtensionHooks::Show, EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleHideTemporaryActors", "Hide Temporary Actors"),
			LOCTEXT("ToggleHideTemporaryActorsToolTip", "When enabled, hides temporary/run-time Actors."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FActorBrowsingMode::OnToggleHideTemporaryActors),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FActorBrowsingMode::ShouldHideTemporaryActors)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleHideLevelInstanceContent", "Hide Level Instance Content"),
			LOCTEXT("ToggleHideLevelInstancesToolTip", "When enabled, hides all level instance content."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FActorBrowsingMode::OnToggleHideLevelInstanceHierarchy),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FActorBrowsingMode::ShouldHideLevelInstanceHierarchy)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}));

	Extender->AddMenuExtension(SceneOutliner::ExtensionHooks::Show, EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection("OutlinerSelectionOptions", LOCTEXT("OptionsHeading", "Options"));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AlwaysFrameSelectionLabel", "Always Frame Selection"),
			LOCTEXT("AlwaysFrameSelectionTooltip", "When enabled, selecting an Actor in the Viewport also scrolls to that Actor in the Outliner."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FActorBrowsingMode::OnToggleAlwaysFrameSelection),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FActorBrowsingMode::ShouldAlwaysFrameSelection)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("FolderDoubleClickToggleCurrentFolderLabel", "Double Click toggles Current Folder"),
			LOCTEXT(
				"FolderDoubleClickToggleCurrentFolderTooltip",
				"When enabled, double clicking on a folder will result in it toggling its Current Folder state"
				"instead of its expansion."
			),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FActorBrowsingMode::OnToggleFolderDoubleClickMarkCurrentFolder),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FActorBrowsingMode::DoesFolderDoubleClickMarkCurrentFolder)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShouldUpdateContentWhileInPIEFocusedLabel", "Update In PIE"),
			LOCTEXT("ShouldUpdateContentWhileInPIEFocusedLabelTooltip", "When enabled, the Outliner will update in PIE."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FActorBrowsingMode::OnToggleShouldUpdateContentWhileInPIEFocused),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FActorBrowsingMode::ShouldUpdateContentWhileInPIEFocused),
				FIsActionButtonVisible::CreateLambda([this]() { return (WorldPartitionEditorModule ? !WorldPartitionEditorModule->GetDisablePIE() : true); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CollapseOutlinerTreeOnNewSelectionLabel", "Collapse Hierarchy on Viewport Selection"),
			LOCTEXT("CollapseOutlinerTreeOnNewSelectionTooltip", "When enabled, all items except the one that was just selected are collapsed."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FActorBrowsingMode::OnToggleCollapseOutlinerTreeOnNewSelection),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FActorBrowsingMode::CollapseOutlinerTreeOnNewSelection)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("World", LOCTEXT("ShowWorldHeading", "World"));
		
		MenuBuilder.AddSubMenu(
			LOCTEXT("ChooseWorldSubMenu", "Choose World"),
			LOCTEXT("ChooseWorldSubMenuToolTip", "Choose the world to display in the outliner."),
			FNewMenuDelegate::CreateRaw(this, &FActorMode::BuildWorldPickerMenu)
		);
		
		MenuBuilder.EndSection();
	}));
}

TSharedRef<FSceneOutlinerFilter> FActorBrowsingMode::CreateShowOnlySelectedActorsFilter()
{
	auto IsActorSelected = [](const AActor* InActor)
	{
		return InActor && InActor->IsSelected();
	};
	return MakeShareable(new FActorFilter(FActorTreeItem::FFilterPredicate::CreateStatic(IsActorSelected), FSceneOutlinerFilter::EDefaultBehaviour::Fail, FActorTreeItem::FFilterPredicate::CreateStatic(IsActorSelected)));
}

TSharedRef<FSceneOutlinerFilter> FActorBrowsingMode::CreateIsInCurrentLevelFilter()
{
	return MakeShareable(new TSceneOutlinerPredicateFilter<ISceneOutlinerTreeItem>(ISceneOutlinerTreeItem::FFilterPredicate::CreateStatic([](const ISceneOutlinerTreeItem& InItem)
		{
			if (const FActorTreeItem* ActorItem = InItem.CastTo<FActorTreeItem>())
			{
				if (const AActor* Actor = ActorItem->Actor.Get(); Actor && Actor->GetWorld())
				{
					if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
					{
						if (LevelInstance->IsEditing())
						{
							return Actor->GetWorld()->GetCurrentLevel() == LevelInstance->GetLoadedLevel();
						}
					}

					return Actor->GetLevel()->IsCurrentLevel();
				}
			}
			else if (const FActorFolderTreeItem* FolderItem = InItem.CastTo<FActorFolderTreeItem>())
			{
				if (const UWorld* FolderWorld = FolderItem->World.Get())
				{
					if (ULevel* FolderLevel = FolderItem->GetFolder().GetRootObjectAssociatedLevel())
					{
						return FolderLevel->IsCurrentLevel();
					}
				}
			}
			else if (const FComponentTreeItem* ComponentItem = InItem.CastTo<FComponentTreeItem>())
			{
				if (const UActorComponent* Component = ComponentItem->Component.Get(); Component && Component->GetComponentLevel())
				{
					return Component->GetComponentLevel()->IsCurrentLevel();
				}
			}
			
			return false;
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FActorBrowsingMode::CreateIsInCurrentDataLayersFilter()
{
	return MakeShareable(new FActorFilter(FActorTreeItem::FFilterPredicate::CreateStatic([](const AActor* InActor)
		{
			const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InActor->GetWorld());
			if (!DataLayerManager || DataLayerManager->GetActorEditorContextDataLayers().IsEmpty())
			{
				return true;
			}

			for (const UDataLayerInstance* DataLayerInstance : InActor->GetDataLayerInstances())
			{
				if (DataLayerInstance->IsInActorEditorContext())
				{
					return true;
				}
			}

			return false;
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FActorBrowsingMode::CreateIsInCurrentContentBundleFilter()
{
	return MakeShareable(new FActorFilter(FActorTreeItem::FFilterPredicate::CreateLambda([this](const AActor* InActor)
		{
			if (!WorldPartitionEditorModule || !WorldPartitionEditorModule->IsEditingContentBundle())
			{
				return true;
			}
			return InActor->GetContentBundleGuid().IsValid() && WorldPartitionEditorModule->IsEditingContentBundle(InActor->GetContentBundleGuid());
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FActorBrowsingMode::CreateHideComponentsFilter()
{
	return MakeShared<TSceneOutlinerPredicateFilter<FComponentTreeItem>>(TSceneOutlinerPredicateFilter<FComponentTreeItem>(
		FComponentTreeItem::FFilterPredicate::CreateStatic([](const UActorComponent*) { return false; }),
		FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FActorBrowsingMode::CreateHideUnloadedActorsFilter()
{
	return MakeShareable(new FActorDescFilter(FActorDescTreeItem::FFilterPredicate::CreateStatic(
		[](const FWorldPartitionActorDescInstance* ActorDescInstance) { return false; }), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FActorBrowsingMode::CreateHideEmptyFoldersFilter()
{
	return MakeShareable(new FFolderFilter(FFolderTreeItem::FFilterPredicate::CreateStatic(
		[](const FFolder& Folder) { return true; }), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

static const FName DefaultContextBaseMenuName("SceneOutliner.DefaultContextMenuBase");
static const FName DefaultContextMenuName("SceneOutliner.DefaultContextMenu");

void FActorBrowsingMode::RegisterContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(DefaultContextBaseMenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(DefaultContextBaseMenuName);
		Menu->AddDynamicSection("DynamicHierarchySection", FNewToolMenuDelegate::CreateStatic(&FActorBrowsingMode::FillDefaultContextBaseMenu));
	}

	if (!ToolMenus->IsMenuRegistered(DefaultContextMenuName))
	{
		ToolMenus->RegisterMenu(DefaultContextMenuName, DefaultContextBaseMenuName);
	}
}

void FActorBrowsingMode::FillDefaultContextBaseMenu(UToolMenu* InMenu)
{
	USceneOutlinerMenuContext* Context = InMenu->FindContext<USceneOutlinerMenuContext>();
	if (!Context)
	{
		return;
	}

	TSharedPtr<SSceneOutliner> SharedOutliner = Context->SceneOutliner.Pin();
	SSceneOutliner* SceneOutliner = SharedOutliner.Get();
	if (!SceneOutliner)
	{
		return;
	}

	{
		// NOTE: the name "Section" is used in many other places
		FToolMenuSection& Section = InMenu->FindOrAddSection("Section");
		Section.Label = LOCTEXT("HierarchySectionName", "Hierarchy");

		if (Context->bShowParentTree)
		{
			if (Context->NumSelectedItems == 0)
			{
				FSceneOutlinerMenuHelper::AddMenuEntryCreateFolder(Section, *SceneOutliner);
			}
			else
			{
				if (Context->NumSelectedItems == 1)
				{
					SceneOutliner->GetTree().GetSelectedItems()[0]->GenerateContextMenu(InMenu, *SceneOutliner);
				}

				// If we've only got folders selected, show the selection and edit sub menus
				if (Context->NumSelectedItems > 0 && Context->NumSelectedFolders == Context->NumSelectedItems)
				{
					Section.AddSubMenu(
						"SelectSubMenu",
						LOCTEXT("SelectSubmenu", "Select"),
						LOCTEXT("SelectSubmenu_Tooltip", "Select the contents of the current selection"),
						FNewToolMenuDelegate::CreateSP(SceneOutliner, &SSceneOutliner::FillSelectionSubMenu));
				}
			}
		}
	}
	
	{
		// We always create a section here, even if there is no parent so that clients can still extend the menu
		FToolMenuSection& MainSection = InMenu->AddSection("MainSection", LOCTEXT("OutlinerSectionName", "Outliner"));

		// Don't add any of these menu items if we're not showing the parent tree
		// Can't move worlds or level blueprints
		if (Context->bShowParentTree && Context->NumSelectedItems > 0 && Context->NumWorldsSelected == 0)
		{
			MainSection.AddSubMenu(
				"MoveActorsTo",
				LOCTEXT("MoveActorsTo", "Move To"),
				LOCTEXT("MoveActorsTo_Tooltip", "Move selection to another folder"),
				FNewToolMenuDelegate::CreateSP(SceneOutliner, &SSceneOutliner::FillFoldersSubMenu));
		}

		if (Context->bShowParentTree && Context->NumSelectedItems > 0)
		{
			// Only add the menu option to wp levels
			if (!Context->bRepresentingGameWorld && Context->bRepresentingPartitionedWorld)
			{
				MainSection.AddMenuEntry(
					"ForceLoadItems",
					LOCTEXT("ForceLoad", "Force Load"),
					LOCTEXT("ForceLoadTooltip", "Keep the selected items loaded in the editor even when they don't overlap a loaded World Partition region"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(SceneOutliner, &SSceneOutliner::PinSelectedItems),
						FCanExecuteAction::CreateLambda([SceneOutliner, Context]()
				{
					if (Context->NumPinnedItems != Context->NumSelectedItems || Context->NumSelectedFolders > 0)
					{
						return SceneOutliner->CanPinSelectedItems();
					}
					return false;
				})));

				MainSection.AddMenuEntry(
					"ReleaseForceLoadItems",
					LOCTEXT("ReleaseForceLoad", "Release Force Load"),
					LOCTEXT("ReleaseForceLoadTooltip", "Allow the World Partition system to load and unload the selected items automatically"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(SceneOutliner, &SSceneOutliner::UnpinSelectedItems),
						FCanExecuteAction::CreateLambda([SceneOutliner, Context]()
				{
					if (Context->NumPinnedItems != 0 || Context->NumSelectedFolders > 0)
					{
						return SceneOutliner->CanUnpinSelectedItems();
					}
					return false;
				})));

			}
		}

		if (Context->NumSelectedItems > 0)
		{
			SceneOutliner->AddSourceControlMenuOptions(InMenu);
		}
	}

	if (Context->bShowParentTree)
	{
		FToolMenuSection& ActorEditorContextSection = InMenu->AddSection("ActorEditorContextSection", LOCTEXT("ActorEditorContextSectionName", "Actor Editor Context"));

		if ((Context->NumSelectedItems == 1) && (Context->NumSelectedFolders == 1))
		{
			ActorEditorContextSection.AddMenuEntry(
				"MakeCurrentFolder",
				LOCTEXT("MakeCurrentFolder", "Make Current Folder"),
				FText(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([SceneOutliner]()
					{
						const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
						if (Selection.SelectedItems.Num() == 1)
						{
							FSceneOutlinerTreeItemPtr Item = Selection.SelectedItems[0].Pin();
							if (FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>())
							{
								if (FolderItem->World.IsValid())
								{
									const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MakeCurrentActorFolder", "Make Current Actor Folder"));
									FActorFolders::Get().SetActorEditorContextFolder(*FolderItem->World, FolderItem->GetFolder());
								}
							}
						}
					}),
					FCanExecuteAction::CreateLambda([SceneOutliner]
					{
						const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
						if (Selection.SelectedItems.Num() == 1)
						{
							FSceneOutlinerTreeItemPtr Item = Selection.SelectedItems[0].Pin();
							if (FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>())
							{
								return FolderItem->World.IsValid() &&
									(FolderItem->World->GetCurrentLevel() == FolderItem->GetFolder().GetRootObjectAssociatedLevel()) &&
									(FActorFolders::Get().GetActorEditorContextFolder(*FolderItem->World) != FolderItem->GetFolder());
							}
						}
						return false;
					})
				)
			);
		}

		const FActorBrowsingMode* Mode = static_cast<const FActorBrowsingMode*>(SceneOutliner->GetMode());
		check(Mode);

		ActorEditorContextSection.AddMenuEntry(
			"ClearCurrentFolder",
			LOCTEXT("ClearCurrentFolder", "Clear Current Folder"),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([Mode]()
				{
					if (Mode->RepresentingWorld.IsValid())
					{
						const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClearCurrentActorFolder", "Clear Current Actor Folder"));
						FActorFolders::Get().SetActorEditorContextFolder(*Mode->RepresentingWorld.Get(), FFolder::GetWorldRootFolder(Mode->RepresentingWorld.Get()));
					}
				}),
				FCanExecuteAction::CreateLambda([Mode]
				{
					return Mode->RepresentingWorld.IsValid() && !FActorFolders::Get().GetActorEditorContextFolder(*Mode->RepresentingWorld.Get()).IsNone();
				})
			)
		);
	}
}

TSharedPtr<SWidget> FActorBrowsingMode::BuildContextMenu()
{
	FActorBrowsingMode::RegisterContextMenu();

	const FSceneOutlinerItemSelection ItemSelection(SceneOutliner->GetSelection());

	USceneOutlinerMenuContext* ContextObject = NewObject<USceneOutlinerMenuContext>();
	ContextObject->SceneOutliner = StaticCastSharedRef<SSceneOutliner>(SceneOutliner->AsShared());
	ContextObject->bShowParentTree = SceneOutliner->GetSharedData().bShowParentTree;
	ContextObject->NumSelectedItems = ItemSelection.Num();
	ContextObject->NumSelectedFolders = ItemSelection.Num<FFolderTreeItem>();
	ContextObject->NumWorldsSelected = ItemSelection.Num<FWorldTreeItem>();
	ContextObject->bRepresentingGameWorld = bRepresentingWorldGameWorld;
	ContextObject->bRepresentingPartitionedWorld = bRepresentingWorldPartitionedWorld;

	int32 NumPinnedItems = 0;
	if (RepresentingWorld.IsValid())
	{
		if (const UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			ItemSelection.ForEachItem<IActorBaseTreeItem>([WorldPartition, &NumPinnedItems](const IActorBaseTreeItem& ActorItem)
				{
					if (WorldPartition->IsActorPinned(ActorItem.GetGuid()))
					{
						++NumPinnedItems;
					}
					return true;
				});
		}
	}
	ContextObject->NumPinnedItems = NumPinnedItems;

	FToolMenuContext Context(ContextObject);

	FName MenuName = DefaultContextMenuName;
	SceneOutliner->GetSharedData().ModifyContextMenu.ExecuteIfBound(MenuName, Context);

	// Build up the menu for a selection
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

	for (const FToolMenuSection& Section : Menu->Sections)
	{
		if (Section.Blocks.Num() > 0)
		{
			return ToolMenus->GenerateWidget(Menu);
		}
	}

	return nullptr;
}

TSharedPtr<SWidget> FActorBrowsingMode::CreateContextMenu()
{
	TArray<AActor*> SelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

	// Make sure that no components are selected
	if (GEditor->GetSelectedComponentCount() > 0)
	{
		// We want to be able to undo to regain the previous component selection
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingOnActorsContextMenu", "Clicking on Actors (context menu)"));
		USelection* ComponentSelection = GEditor->GetSelectedComponents();
		ComponentSelection->Modify(false);
		ComponentSelection->DeselectAll();

		GUnrealEd->UpdatePivotLocationForSelection();
		GEditor->RedrawLevelEditingViewports(false);
	}

	return BuildContextMenu();
}

void FActorBrowsingMode::OnItemAdded(FSceneOutlinerTreeItemPtr Item)
{
	if (const FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
	{
		// We incremented the count regardless of Flags.bIsFilteredOut because the count should match what the user sees, which includes things like
		// actors which don't pass the filter themselves but are force shown by children being visible.
		++FilteredActorCount;
		
		if (!Item->Flags.bIsFilteredOut)
		{
			// Synchronize selection
			if (GEditor->GetSelectedActors()->IsSelected(ActorItem->Actor.Get()))
			{
				SceneOutliner->SetItemSelection(Item, true);
			}
		}
	}
	else if (Item->IsA<FActorDescTreeItem>())
	{
		++FilteredActorCount;
		++FilteredUnloadedActorCount;
	}
}

void FActorBrowsingMode::OnItemRemoved(FSceneOutlinerTreeItemPtr Item)
{
	if (Item->IsA<FActorTreeItem>())
	{
		--FilteredActorCount;
	}
	else if (Item->IsA<FActorDescTreeItem>())
	{
		--FilteredActorCount;
		--FilteredUnloadedActorCount;
	}
}

void FActorBrowsingMode::OnComponentsUpdated()
{
	UE_LOGF(LogSceneOutliner, VeryVerbose, "FullRefresh requested by FActorBrowsingMode::OnComponentsUpdated");

	SceneOutliner->FullRefresh();
}

void FActorBrowsingMode::OnLevelActorDeleted(AActor* Actor)
{
	ApplicableActors.Remove(Actor);
}

void FActorBrowsingMode::OnSelectUnloadedActors(const TArray<FGuid>& ActorGuids)
{
	if (RepresentingWorld.IsValid())
	{
		if (UWorldPartition* WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			TArray<FSceneOutlinerTreeItemPtr> ItemsToSelect;
			ItemsToSelect.Reserve(ActorGuids.Num());
			for (const FGuid& ActorGuid : ActorGuids)
			{
				if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid))
				{
					if (FSceneOutlinerTreeItemPtr ItemPtr = SceneOutliner->GetTreeItem(FActorDescTreeItem::ComputeTreeItemID(ActorDescInstance->GetGuid(), ActorDescInstance->GetContainerInstance())))
					{
						ItemsToSelect.Add(ItemPtr);
					}
				}
			}

			if (ItemsToSelect.Num())
			{
				SceneOutliner->SetItemSelection(ItemsToSelect, true);
				SceneOutliner->ScrollItemIntoView(ItemsToSelect.Last());

				if (const FActorDescTreeItem* ActorDescItem = ItemsToSelect.Last()->CastTo<FActorDescTreeItem>())
				{
					ActorDescItem->FocusActorBounds();
				}
			}
		}
	}
}

void FActorBrowsingMode::OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	ApplicableUnloadedActors.Remove(InActorDescInstance);
}

void FActorBrowsingMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	TSet<AActor*> OutlinerSelectedActors(Selection.GetData<AActor*>(SceneOutliner::FActorSelector()));
	TSet<AActor*> ActorsToSelect;
	ActorsToSelect.Reserve(OutlinerSelectedActors.Num());

	SynchronizeSelectedActorDescs();

	USelection* ActorSelection = GEditor->GetSelectedActors();
	if (UTypedElementSelectionSet* SelectionSet = ActorSelection->GetElementSelectionSet())
	{
		TSet<AActor*> EditorSelectedActors(SelectionSet->GetSelectedObjects<AActor>());
		
		const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
		if (ModifierKeys.IsControlDown())
		{
			for (AActor* SelectedActor : EditorSelectedActors)
			{
				// If we can't find item in tree == item is filtered out, since CTRL is held, add it to current selection
				FSceneOutlinerTreeItemPtr Outliner = SceneOutliner->GetTreeItem(SelectedActor);
				if (!Outliner && SelectedActor != nullptr)
				{
					OutlinerSelectedActors.Add(SelectedActor);
				}
			}
		}

		bool bChanged = false;
		bool bAnyInPIE = false;
		for (AActor* Actor : OutlinerSelectedActors)
		{
			if (!bAnyInPIE && Actor && Actor->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor))
			{
				bAnyInPIE = true;
			}

			if (!EditorSelectedActors.Contains(Actor))
			{
				bChanged = true;
			}

			// Allow selection of Sub Roots only if Roots aren't in the outliners selection
			AActor* RootSelectionParent = Actor ? Actor->GetRootSelectionParent() : nullptr;
			if (!RootSelectionParent || !OutlinerSelectedActors.Contains(RootSelectionParent))
			{
				ActorsToSelect.Add(Actor);
			}
		}

		for (const AActor* SelectedActor : EditorSelectedActors)
		{
			if (!bAnyInPIE && SelectedActor && SelectedActor->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor))
			{
				bAnyInPIE = true;
			}

			if (!ActorsToSelect.Contains(SelectedActor))
			{
				// Actor has been deselected
				bChanged = true;

				// If actor was a group actor, remove its members from the ActorsToSelect list
				if (const AGroupActor* DeselectedGroupActor = Cast<AGroupActor>(SelectedActor))
				{
					TArray<AActor*> GroupActors;
					DeselectedGroupActor->GetGroupActors(GroupActors);

					for (AActor* GroupActor : GroupActors)
					{
						ActorsToSelect.Remove(GroupActor);
					}
				}
			}
		}

		// If there's a discrepancy, update the selected actors to reflect this list.
		if (bChanged)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingOnActors", "Clicking on Actors"), !bAnyInPIE);

			TArray<FTypedElementHandle> ElementsToSelect;
			for (auto* Actor : ActorsToSelect)
			{
				UE_LOGF(LogActorBrowser, Verbose, "Clicking on Actor (world outliner): %ls (%ls)", *Actor->GetClass()->GetName(), *Actor->GetActorLabel());
				ElementsToSelect.Add(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor));
			}

			{
				const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
					.SetAllowHidden(true)
					.SetWarnIfLocked(false)
					.SetAllowLegacyNotifications(false)
					.SetAllowSubRootSelection(true);

				FTypedElementList::FScopedClearNewPendingChange ClearNewPendingChange = SelectionSet->GetScopedClearNewPendingChange();
				SelectionSet->SetSelection(ElementsToSelect, SelectionOptions);
				SelectionSet->NotifyPendingChanges();
			}

			// Set this outliner as the most recently interacted with
			SetAsMostRecentOutliner();
		}

		SceneOutliner->RefreshSelection();
	}
}

void FActorBrowsingMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item)
{
	if (const FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
	{
		if (AActor* Actor = ActorItem->Actor.Get())
		{
			ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor);
			if (LevelInstance && FSlateApplication::Get().GetModifierKeys().IsAltDown())
			{
				if (LevelInstance->CanEnterEdit())
				{
					LevelInstance->EnterEdit();
				}
				else if (LevelInstance->CanExitEdit())
				{
					LevelInstance->ExitEdit();
				}
			}
			else if (Item->CanInteract())
			{
				FSceneOutlinerItemSelection Selection(SceneOutliner->GetSelection());
				if (Selection.Has<FActorTreeItem>())
				{
					const bool bActiveViewportOnly = false;
					GEditor->MoveViewportCamerasToActor(Selection.GetData<AActor*>(SceneOutliner::FActorSelector()), bActiveViewportOnly);
				}
			}
			else
			{
				const bool bActiveViewportOnly = false;
				GEditor->MoveViewportCamerasToActor(*Actor, bActiveViewportOnly);
			}
		}
	}
	else if (const FActorDescTreeItem* ActorDescItem = Item->CastTo<FActorDescTreeItem>())
	{
		ActorDescItem->FocusActorBounds();
	}
	else if (const FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>())
	{
		if (DoesFolderDoubleClickMarkCurrentFolder())
		{
			if (UWorld* World = FolderItem->World.Get())
			{
				const FScopedTransaction Transaction(
					LOCTEXT("ToggleCurrentActorFolder", "Toggle Current Actor Folder")
				);

				const FFolder CurrentContextFolder = FActorFolders::Get().GetActorEditorContextFolder(*World);
				if (CurrentContextFolder == FolderItem->GetFolder())
				{
					FActorFolders::Get().SetActorEditorContextFolder(*World, FFolder::GetWorldRootFolder(World));
				}
				else
				{
					FActorFolders::Get().SetActorEditorContextFolder(*World, FolderItem->GetFolder());
				}
			}
		}
	}
}

bool FActorBrowsingMode::HasCustomFolderDoubleClick() const
{
	return DoesFolderDoubleClickMarkCurrentFolder();
}

void FActorBrowsingMode::OnToggleFolderDoubleClickMarkCurrentFolder()
{
	if (FActorBrowsingModeConfig* Settings = GetMutableConfig())
	{
		if (Settings->FolderDoubleClickMethod == EActorBrowsingFolderDoubleClickMethod::ToggleCurrentFolder)
		{
			Settings->FolderDoubleClickMethod = EActorBrowsingFolderDoubleClickMethod::ToggleExpansion;
		}
		else
		{
			Settings->FolderDoubleClickMethod = EActorBrowsingFolderDoubleClickMethod::ToggleCurrentFolder;
		}

		SaveConfig();
	}
}

bool FActorBrowsingMode::DoesFolderDoubleClickMarkCurrentFolder() const
{
	if (const FActorBrowsingModeConfig* Settings = GetConstConfig())
	{
		return Settings->FolderDoubleClickMethod == EActorBrowsingFolderDoubleClickMethod::ToggleCurrentFolder;
	}

	return false;
}

void FActorBrowsingMode::OnToggleShouldUpdateContentWhileInPIEFocused()
{
	if (FActorBrowsingModeConfig* Settings = GetMutableConfig())
	{
		Settings->bShouldUpdateContentWhileInPIEFocused = !Settings->bShouldUpdateContentWhileInPIEFocused;
		bShouldUpdateContentWhileInPIEFocused = Settings->bShouldUpdateContentWhileInPIEFocused;
		SaveConfig();
	}
}

bool FActorBrowsingMode::ShouldUpdateContentWhileInPIEFocused() const
{
	return bShouldUpdateContentWhileInPIEFocused;
}

void FActorBrowsingMode::OnToggleCollapseOutlinerTreeOnNewSelection()
{
	if (FActorBrowsingModeConfig* Settings = GetMutableConfig())
	{
		Settings->bCollapseOutlinerTreeOnNewSelection = !Settings->bCollapseOutlinerTreeOnNewSelection;
		bCollapseOutlinerTreeOnNewSelection = Settings->bCollapseOutlinerTreeOnNewSelection;
		SaveConfig();
	}
}

bool FActorBrowsingMode::CollapseOutlinerTreeOnNewSelection() const
{
	return bCollapseOutlinerTreeOnNewSelection;
}

void FActorBrowsingMode::OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType)
{
	// Start batching selection changes
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();

	// Select actors (and only the actors) that match the filter text
	const bool bNoteSelectionChange = false;
	const bool bDeselectBSPSurfs = false;
	const bool WarnAboutManyActors = true;
	GEditor->SelectNone(bNoteSelectionChange, bDeselectBSPSurfs, WarnAboutManyActors);
	for (AActor* Actor : Selection.GetData<AActor*>(SceneOutliner::FActorSelector()))
	{
		const bool bShouldSelect = true;
		const bool bSelectEvenIfHidden = false;
		GEditor->SelectActor(Actor, bShouldSelect, bNoteSelectionChange, bSelectEvenIfHidden);
	}

	// Commit selection changes
	GEditor->GetSelectedActors()->EndBatchSelectOperation(/*bNotify*/false);

	// Fire selection changed event
	GEditor->NoteSelectionChange();

	// Set keyboard focus to the SceneOutliner, so the user can perform keyboard commands that interact
	// with selected actors (such as Delete, to delete selected actors.)
	SceneOutliner->SetKeyboardFocus();

	SetAsMostRecentOutliner();
}

void FActorBrowsingMode::OnItemPassesFilters(const ISceneOutlinerTreeItem& Item)
{
	if (const FActorTreeItem* const ActorItem = Item.CastTo<FActorTreeItem>())
	{
		ApplicableActors.Add(ActorItem->Actor);
	}
	else if (const FActorDescTreeItem* const ActorDescItem = Item.CastTo<FActorDescTreeItem>(); ActorDescItem && ActorDescItem->IsValid())
	{
		ApplicableUnloadedActors.Add(ActorDescItem->ActorDescHandle);
	}
}

FReply FActorBrowsingMode::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	const FInputChord CheckChord( InKeyEvent.GetKey(), EModifierKey::FromBools(ModifierKeys.IsControlDown(), ModifierKeys.IsAltDown(), ModifierKeys.IsShiftDown(), ModifierKeys.IsCommandDown()) );

	// Use the keyboard shortcut bound to 'Focus Viewport To Selection'
	if (FEditorViewportCommands::Get().FocusViewportToSelection->HasActiveChord(CheckChord))
	{
		if (Selection.Num() == 1)
		{
			FSceneOutlinerTreeItemPtr ItemToFocus = Selection.SelectedItems[0].Pin();

			if (ItemToFocus.IsValid())
			{
				SceneOutliner->ScrollItemIntoView(ItemToFocus);
			}
		}
	}

	// Always return Unhandled here even if it entered the previous if so that the level editor viewport can handle the FocusViewport command as well
	return FReply::Unhandled();
}

bool FActorBrowsingMode::CanDelete() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders > 0 && NumberOfFolders == ItemSelection.Num());
}

bool FActorBrowsingMode::CanRename() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders == 1 && NumberOfFolders == ItemSelection.Num());
}

bool FActorBrowsingMode::CanRenameItem(const ISceneOutlinerTreeItem& Item) const
{
	// Can only rename actor and folder items when in actor browsing mode
	return (Item.IsValid() && (Item.IsA<FActorTreeItem>() || Item.IsA<FFolderTreeItem>()));
}

bool FActorBrowsingMode::CanCut() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders > 0 && NumberOfFolders == ItemSelection.Num());
}

bool FActorBrowsingMode::CanCopy() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders > 0 && NumberOfFolders == ItemSelection.Num());
}

bool FActorBrowsingMode::CanPaste() const
{
	return CanPasteFoldersOnlyFromClipboard();
}

bool FActorBrowsingMode::HasErrors() const
{
	if (RepresentingWorld.IsValid() && !bRepresentingWorldGameWorld && bRepresentingWorldPartitionedWorld && WorldPartitionEditorModule)
	{
		return WorldPartitionEditorModule->HasErrors(RepresentingWorld.Get());
	}

	return false;
}

FText FActorBrowsingMode::GetErrorsText() const
{
	return LOCTEXT("WorldHasInvalidActorFiles", "The world contains invalid actor files. Click the Repair button to repair them.");
}

void FActorBrowsingMode::RepairErrors() const
{
	if (RepresentingWorld.IsValid() && !bRepresentingWorldGameWorld && bRepresentingWorldPartitionedWorld && WorldPartitionEditorModule)
	{
		WorldPartitionEditorModule->RepairErrors(RepresentingWorld.Get());
	}
}

void FActorBrowsingMode::BindCommands(const TSharedRef<FUICommandList>& OutCommandList)
{
	OutCommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateRaw(this, &FActorBrowsingMode::OnExecuteRename ),
		FCanExecuteAction::CreateRaw(this, &FActorBrowsingMode::CanExecuteRename));

	// Override DELETE command with custom delete
	if (SceneOutliner && SceneOutliner->GetSharedData().CustomDelete.IsBound())
	{
		OutCommandList->MapAction(
			FGenericCommands::Get().Delete,
			FExecuteAction::CreateRaw(this, &FActorBrowsingMode::OnExecuteDelete),
			FCanExecuteAction::CreateRaw(this, &FActorBrowsingMode::CanDelete)
		);
	}
	
	OutCommandList->MapAction(
		FActorBrowsingModeCommands::Get().Refresh,
		FExecuteAction::CreateRaw(this, &FActorBrowsingMode::OnExecuteRefresh));

	FInputBindingManager::Get().RegisterCommandList(FActorBrowsingModeCommands::Get().GetContextName(), OutCommandList);
}

bool FActorBrowsingMode::CanPasteFoldersOnlyFromClipboard() const
{
	// Intentionally not checking if the level is locked/hidden here, as it's better feedback for the user if they attempt to paste
	// and get the message explaining why it's failed, than just not having the option available to them.
	FString PasteString;
	FPlatformApplicationMisc::ClipboardPaste(PasteString);
	return PasteString.StartsWith("BEGIN FOLDERLIST");
}

void FActorBrowsingMode::OnExecuteDelete()
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (SceneOutliner->GetSharedData().CustomDelete.IsBound())
	{
		SceneOutliner->GetSharedData().CustomDelete.Execute(Selection.SelectedItems);
	}
	else
	{
		if (RepresentingWorld.IsValid())
		{
			GUnrealEd->Exec(RepresentingWorld.Get(), TEXT("DELETE"));
		}
	}
}

void FActorBrowsingMode::OnExecuteRefresh()
{
	SceneOutliner->FullRefresh();
}

void FActorBrowsingMode::OnExecuteRename()
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();

		if (ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract())
		{
			SceneOutliner->SetPendingRenameItem(ItemToRename);
			SceneOutliner->ScrollItemIntoView(ItemToRename);
		}
	}
}

bool FActorBrowsingMode::CanExecuteRename()
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();
		return ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract();
	}
	return false;
}

void FActorBrowsingMode::SynchronizeSelectedActorDescs()
{
	if (UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(RepresentingWorld.Get()))
	{
		const FSceneOutlinerItemSelection Selection = SceneOutliner->GetSelection();
		TArray<FWorldPartitionHandle> SelectedActorHandles = Selection.GetData<FWorldPartitionHandle>(SceneOutliner::FActorHandleSelector());

		WorldPartitionSubsystem->SelectedActorHandles.Empty();
		for (const FWorldPartitionHandle& ActorHandle : SelectedActorHandles)
		{
			WorldPartitionSubsystem->SelectedActorHandles.Add(ActorHandle);
		}
	}
}

FFolder FActorBrowsingMode::CreateNewFolder()
{
	if (TStrongObjectPtr<UWorld> World = RepresentingWorld.Pin())
	{
		const FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));
		TArray<FFolder> SelectedFolders = SceneOutliner->GetSelection().GetData<FFolder>(SceneOutliner::FFolderPathSelector());
		const FFolder NewFolderName = FActorFolders::Get().GetDefaultFolderForSelection(*World, &SelectedFolders);
		FActorFolders::Get().CreateFolderContainingSelection(*World, NewFolderName);
		return NewFolderName;
	}

	return FFolder();
}

FFolder FActorBrowsingMode::GetFolder(const FFolder& ParentPath, const FName& LeafName)
{
	// Return a unique folder under the provided parent path & root object and using the provided leaf name
	if (TStrongObjectPtr<UWorld> World = RepresentingWorld.Pin())
	{
		return FActorFolders::Get().GetFolderName(*World, ParentPath, LeafName);
	}
	return FFolder();
}

bool FActorBrowsingMode::CreateFolder(const FFolder& NewPath)
{
	if (TStrongObjectPtr<UWorld> World = RepresentingWorld.Pin())
	{
		return FActorFolders::Get().CreateFolder(*World, NewPath);
	}
	return false;
}

bool FActorBrowsingMode::ReparentItemToFolder(const FFolder& FolderPath, const FSceneOutlinerTreeItemPtr& Item)
{
	if (FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
	{
		// Make sure actor has the same root object before updating path
		if (ActorItem->Actor->GetFolderRootObject() == FolderPath.GetRootObject())
		{
			ActorItem->Actor->SetFolderPath_Recursively(FolderPath.GetPath());
			return true;
		}
	}
	return false;
}

namespace ActorBrowsingModeUtils
{
	static void RecursiveFolderExpandChildren(SSceneOutliner* SceneOutliner, const FSceneOutlinerTreeItemPtr& Item)
	{
		if (Item.IsValid())
		{
			for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : Item->GetChildren())
			{
				FSceneOutlinerTreeItemPtr ChildPtr = Child.Pin();
				SceneOutliner->SetItemExpansion(ChildPtr, true);
				RecursiveFolderExpandChildren(SceneOutliner, ChildPtr);
			}
		}
	}

	static void RecursiveActorSelect(SSceneOutliner* SceneOutliner, const FSceneOutlinerTreeItemPtr& Item, bool bSelectImmediateChildrenOnly)
	{
		if (Item.IsValid())
		{
			// If the current item is an actor, ensure to select it as well
			if (FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
			{
				if (AActor* Actor = ActorItem->Actor.Get())
				{
					GEditor->SelectActor(Actor, true, false);
				}
			}
			// Select all children
			for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : Item->GetChildren())
			{
				FSceneOutlinerTreeItemPtr ChildPtr = Child.Pin();
				if (ChildPtr.IsValid())
				{
					if (FActorTreeItem* ActorItem = ChildPtr->CastTo<FActorTreeItem>())
					{
						if (AActor* Actor = ActorItem->Actor.Get())
						{
							GEditor->SelectActor(Actor, true, false);
						}
					}
					else if (FFolderTreeItem* FolderItem = ChildPtr->CastTo<FFolderTreeItem>())
					{
						SceneOutliner->SetItemSelection(FolderItem->AsShared(), true);
					}

					if (!bSelectImmediateChildrenOnly)
					{
						for (const TWeakPtr<ISceneOutlinerTreeItem>& Grandchild : ChildPtr->GetChildren())
						{
							RecursiveActorSelect(SceneOutliner, Grandchild.Pin(), bSelectImmediateChildrenOnly);
						}
					}
				}
			}
		}
	}

	static void RecursiveAddItemsToActorGuidList(const TArray<FSceneOutlinerTreeItemPtr>& Items, TArray<FGuid>& List, bool bSearchForHiddenUnloadedActors)
	{
		// In the case where we want the list of unloaded actors under a folder and the option to hide unloaded actors is enabled, we need to find them through FActorFolders 
		TMap<UWorld*, TSet<FName>> UnloadedActorsFolderPaths;

		for (const FSceneOutlinerTreeItemPtr& Item : Items)
		{
			if (const FActorDescTreeItem* const ActorDescTreeItem = Item->CastTo<FActorDescTreeItem>())
			{
				List.Add(ActorDescTreeItem->GetGuid());
			}
			else if (const FActorTreeItem* const ActorTreeItem = Item->CastTo<FActorTreeItem>())
			{
				if (ActorTreeItem->Actor.IsValid())
				{
					List.Add(ActorTreeItem->Actor->GetActorGuid());
				}
			}
			else if (const FActorFolderTreeItem* const ActorFolderTreeItem = Item->CastTo<FActorFolderTreeItem>(); ActorFolderTreeItem && bSearchForHiddenUnloadedActors)
			{
				if (UWorld* FolderWorld = ActorFolderTreeItem->World.Get())
				{
					UnloadedActorsFolderPaths.FindOrAdd(FolderWorld).Add(ActorFolderTreeItem->GetPath());
				}
			}

			TArray<FSceneOutlinerTreeItemPtr> ChildrenItems;
			for (const auto& Child : Item->GetChildren())
			{
				if (Child.IsValid())
				{
					ChildrenItems.Add(Child.Pin());
				}
			}
						
			if (ChildrenItems.Num())
			{
				RecursiveAddItemsToActorGuidList(ChildrenItems, List, bSearchForHiddenUnloadedActors);
			}
		}

		
		for (const TTuple<UWorld*, TSet<FName>>& Pair : UnloadedActorsFolderPaths)
		{
			FActorFolders::ForEachActorDescInstanceInFolders(*Pair.Key, Pair.Value, [&List](const FWorldPartitionActorDescInstance* ActorDescInstance)
			{
				if (!ActorDescInstance->IsLoaded())
				{
					List.Add(ActorDescInstance->GetGuid());
				}
				return true;
			});
		}
	};

	bool CanChangePinnedStates(const TArray<FSceneOutlinerTreeItemPtr>& InItems)
	{
		for (const FSceneOutlinerTreeItemPtr& Item : InItems)
		{
			if (Item->ShouldShowPinnedState())
			{
				return true;
			}
			else if (const FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>(); FolderItem && FolderItem->CanChangeChildrenPinnedState())
			{
				return true;
			}
		}

		return false;
	}
}

void FActorBrowsingMode::SelectFoldersDescendants(const TArray<FFolderTreeItem*>& FolderItems, bool bSelectImmediateChildrenOnly)
{
	// Expand everything before beginning selection
	for (FFolderTreeItem* Folder : FolderItems)
	{
		FSceneOutlinerTreeItemPtr FolderPtr = Folder->AsShared();
		SceneOutliner->SetItemExpansion(FolderPtr, true);
		if (!bSelectImmediateChildrenOnly)
		{
			ActorBrowsingModeUtils::RecursiveFolderExpandChildren(SceneOutliner, FolderPtr);
		}
	}

	// batch selection
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();

	for (FFolderTreeItem* Folder : FolderItems)
	{
		ActorBrowsingModeUtils::RecursiveActorSelect(SceneOutliner, Folder->AsShared(), bSelectImmediateChildrenOnly);
	}

	GEditor->GetSelectedActors()->EndBatchSelectOperation(/*bNotify*/false);
	GEditor->NoteSelectionChange();
}

bool FActorBrowsingMode::CanPinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const
{
	return ActorBrowsingModeUtils::CanChangePinnedStates(InItems);
}

void FActorBrowsingMode::PinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems)
{
	UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition();
	if (!WorldPartition)
	{
		return;
	}

	TArray<FGuid> ActorsToPin;
	// If Unloaded actors are hidden and we are pinning folders we need to find them through FActorFolders
	const bool bSearchForHiddenUnloadedActors = bHideUnloadedActors;
	ActorBrowsingModeUtils::RecursiveAddItemsToActorGuidList(InItems, ActorsToPin, bSearchForHiddenUnloadedActors);

	if (ActorsToPin.Num())
	{
		GEditor->GetSelectedActors()->BeginBatchSelectOperation();
		GEditor->SelectNone(/*bNoteSelectionChange=*/false, /*bDeselectBSPSurfs=*/true);

		WorldPartition->PinActors(ActorsToPin);

		AActor* LastPinnedActor = nullptr;
		for (const FGuid& ActorGuid : ActorsToPin)
		{
			if (FWorldPartitionHandle ActorHandle(WorldPartition, ActorGuid); ActorHandle.IsValid())
			{
				if (AActor* PinnedActor = ActorHandle->GetActor())
				{
					GEditor->SelectActor(PinnedActor, /*bInSelected=*/true, /*bNotify=*/false);
					LastPinnedActor = PinnedActor;
				}
			}
		}

		GEditor->GetSelectedActors()->EndBatchSelectOperation(/*bNotify=*/true);

		if (LastPinnedActor)
		{
			SceneOutliner->OnItemAdded(LastPinnedActor, SceneOutliner::ENewItemAction::ScrollIntoView);
		}
	}
}

bool FActorBrowsingMode::CanUnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const
{
	return ActorBrowsingModeUtils::CanChangePinnedStates(InItems);
}

void FActorBrowsingMode::UnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems)
{
	UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition();
	if (!WorldPartition)
	{
		return;
	}

	TArray<FGuid> ActorsToUnpin;
	// No need to search for hidden unloaded actors when unloading
	const bool bSearchForHiddenUnloadedActors = false;
	ActorBrowsingModeUtils::RecursiveAddItemsToActorGuidList(InItems, ActorsToUnpin, bSearchForHiddenUnloadedActors);

	if (ActorsToUnpin.Num())
	{
		WorldPartition->UnpinActors(ActorsToUnpin);
	}
}

void FActorBrowsingMode::SynchronizeSelection()
{
	FActorModeInteractive::SynchronizeSelection();
	SynchronizeSelectedActorDescs();
}

FCreateSceneOutlinerMode FActorBrowsingMode::CreateFolderPickerMode(const FFolder::FRootObject& InRootObject) const
{
	auto MoveSelectionTo = [this, InRootObject](const FSceneOutlinerTreeItemRef& NewParent)
	{
		if (FWorldTreeItem* WorldItem = NewParent->CastTo<FWorldTreeItem>())
		{
			SceneOutliner->MoveSelectionTo(GetWorldDefaultRootFolder());
		}
		else if (FFolderTreeItem* FolderItem = NewParent->CastTo<FFolderTreeItem>())
		{
			SceneOutliner->MoveSelectionTo(FolderItem->GetFolder());
		}
		else if (FActorTreeItem* ActorItem = NewParent->CastTo<FActorTreeItem>())
		{
			if (FFolder::IsRootObjectValid(InRootObject))
			{
				SceneOutliner->MoveSelectionTo(FFolder(InRootObject));
			}
		}
		else if (FLevelTreeItem* LevelItem = NewParent->CastTo<FLevelTreeItem>())
		{
			if (FFolder::IsRootObjectValid(InRootObject))
			{
				SceneOutliner->MoveSelectionTo(FFolder(InRootObject));
			}
		}
	};

	return FCreateSceneOutlinerMode::CreateLambda([this, MoveSelectionTo, InRootObject](SSceneOutliner* Outliner)
	{
		return new FActorFolderPickingMode(Outliner, FOnSceneOutlinerItemPicked::CreateLambda(MoveSelectionTo), nullptr, InRootObject);
	});
}

void FActorBrowsingMode::OnDuplicateSelected()
{
	GUnrealEd->Exec(RepresentingWorld.Get(), TEXT("DUPLICATE"));
}

void FActorBrowsingMode::OnEditCutActorsBegin()
{
	// If TEv1 interfaces for folders is enabled, we don't want to rely on the legacy delegate based approach
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	if (CVar && CVar->GetBool())
	{
		return;
	}

	// Only a callback in actor browsing mode
	SceneOutliner->CopyFoldersBegin();
	SceneOutliner->DeleteFoldersBegin();
}

void FActorBrowsingMode::OnEditCutActorsEnd()
{
	// If TEv1 interfaces for folders is enabled, we don't want to rely on the legacy delegate based approach
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	if (CVar && CVar->GetBool())
	{
		return;
	}
	// Only a callback in actor browsing mode
	SceneOutliner->CopyFoldersEnd();
	SceneOutliner->DeleteFoldersEnd();
}

void FActorBrowsingMode::OnEditCopyActorsBegin()
{
	// If TEv1 interfaces for folders is enabled, we don't want to rely on the legacy delegate based approach
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	if (CVar && CVar->GetBool())
	{
		return;
	}
	
	// Only a callback in actor browsing mode
	SceneOutliner->CopyFoldersBegin();
}

void FActorBrowsingMode::OnEditCopyActorsEnd()
{
	// If TEv1 interfaces for folders is enabled, we don't want to rely on the legacy delegate based approach
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	if (CVar && CVar->GetBool())
	{
		return;
	}
	
	// Only a callback in actor browsing mode
	SceneOutliner->CopyFoldersEnd();
}

void FActorBrowsingMode::OnEditPasteActorsBegin()
{
	// If TEv1 interfaces for folders is enabled, we don't want to rely on the legacy delegate based approach
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	if (CVar && CVar->GetBool())
	{
		return;
	}
	
	// Only a callback in actor browsing mode
	const TArray<FName> FolderPaths = SceneOutliner->GetClipboardPasteFolders();
	SceneOutliner->PasteFoldersBegin(FolderPaths);
}

void FActorBrowsingMode::OnEditPasteActorsEnd()
{
	// If TEv1 interfaces for folders is enabled, we don't want to rely on the legacy delegate based approach
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	if (CVar && CVar->GetBool())
	{
		return;
	}
	
	// Only a callback in actor browsing mode
	SceneOutliner->PasteFoldersEnd();
}

void FActorBrowsingMode::OnDuplicateActorsBegin()
{
	// If TEv1 interfaces for folders is enabled, we don't want to rely on the legacy delegate based approach
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	if (CVar && CVar->GetBool())
	{
		return;
	}
	
	// Only a callback in actor browsing mode
	FFolder::FRootObject CommonRootObject;
	TArray<FName> SelectedFolderPaths;
	FFolder::GetFolderPathsAndCommonRootObject(SceneOutliner->GetSelection().GetData<FFolder>(SceneOutliner::FFolderPathSelector()), SelectedFolderPaths, CommonRootObject);
	SceneOutliner->PasteFoldersBegin(SelectedFolderPaths);
}

void FActorBrowsingMode::OnDuplicateActorsEnd()
{
	// If TEv1 interfaces for folders is enabled, we don't want to rely on the legacy delegate based approach
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	if (CVar && CVar->GetBool())
	{
		return;
	}
	
	// Only a callback in actor browsing mode
	SceneOutliner->PasteFoldersEnd();
}

void FActorBrowsingMode::OnDeleteActorsBegin()
{
	// If TEv1 interfaces for folders is enabled, we don't want to rely on the legacy delegate based approach
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	if (CVar && CVar->GetBool())
	{
		return;
	}
	
	SceneOutliner->DeleteFoldersBegin();
}

void FActorBrowsingMode::OnDeleteActorsEnd()
{
	// If TEv1 interfaces for folders is enabled, we don't want to rely on the legacy delegate based approach
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	if (CVar && CVar->GetBool())
	{
		return;
	}
	
	SceneOutliner->DeleteFoldersEnd();
}

struct FActorBrowsingModeConfig* FActorBrowsingMode::GetMutableConfig()
{
	FName OutlinerIdentifier = SceneOutliner->GetOutlinerIdentifier();

	if (OutlinerIdentifier.IsNone())
	{
		return nullptr;
	}

	return &UActorBrowserConfig::Get()->ActorBrowsers.FindOrAdd(OutlinerIdentifier);
}


const FActorBrowsingModeConfig* FActorBrowsingMode::GetConstConfig() const
{
	FName OutlinerIdentifier = SceneOutliner->GetOutlinerIdentifier();

	if (OutlinerIdentifier.IsNone())
	{
		return nullptr;
	}

	return UActorBrowserConfig::Get()->ActorBrowsers.Find(OutlinerIdentifier);
}

void FActorBrowsingMode::SaveConfig()
{
	UActorBrowserConfig::Get()->SaveEditorConfig();
}

bool FActorBrowsingMode::CompareItemWithClassName(SceneOutliner::FilterBarType InItem, const TSet<FTopLevelAssetPath>& AssetClassPaths) const
{
	// Type filtering only supported for Actors (and unloaded actors) currently
	if (const FActorTreeItem* ActorItem = InItem.CastTo<FActorTreeItem>())
	{
		AActor* Actor = ActorItem->Actor.Get();

		if(!Actor)
		{
			return false;
		}

		FTopLevelAssetPath AssetClassPath = Actor->GetClass()->GetClassPathName();

		// For Blueprints, we check both the parent type (e.g Pawn/Actor etc) and the Blueprint class itself
		if (UBlueprint* ClassBP = UBlueprint::GetBlueprintFromClass(Actor->GetClass()))
		{
			return AssetClassPaths.Contains(ClassBP->GetClass()->GetClassPathName()) || AssetClassPaths.Contains(AssetClassPath);
		}
		
		return AssetClassPaths.Contains(AssetClassPath);
	}
	else if (const FActorDescTreeItem* ActorDescItem = InItem.CastTo<FActorDescTreeItem>())
	{
		if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
		{
			// For Unloaded Actors, grab the native class 
			FTopLevelAssetPath ClassPath = ActorDescInstance->GetNativeClass();
			return AssetClassPaths.Contains(ClassPath);
		}
	}

	return FActorModeInteractive::CompareItemWithClassName(InItem, AssetClassPaths);
}

#undef LOCTEXT_NAMESPACE
