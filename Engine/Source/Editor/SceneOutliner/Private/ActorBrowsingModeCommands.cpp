// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorBrowsingModeCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "ActorBrowsingModeCommands"

void FActorBrowsingModeCommands::RegisterCommands()
{
	UI_COMMAND(Refresh, "Refresh", "Refresh Outliner", EUserInterfaceActionType::Button, FInputChord(EKeys::F5));
}

#undef LOCTEXT_NAMESPACE
