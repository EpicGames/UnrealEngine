// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FActorBrowsingModeCommands
	: public TCommands<FActorBrowsingModeCommands>
{
public:
	/** Default constructor. */
	FActorBrowsingModeCommands()
		: TCommands<FActorBrowsingModeCommands>( TEXT("ActorBrowsingModeCommands"), NSLOCTEXT("ActorBrowsingModeCommands", "ActorBrowsingModeCommands", "Actor Browsing Mode Commands"), NAME_None, FAppStyle::GetAppStyleSetName() )
	{
	}

	virtual ~FActorBrowsingModeCommands() = default;

public:

	//~ TCommands interface

	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> Refresh;
};
