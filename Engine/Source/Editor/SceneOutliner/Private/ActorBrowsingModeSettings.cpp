// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorBrowsingModeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorBrowsingModeSettings)

TObjectPtr<UActorBrowserConfig> UActorBrowserConfig::Instance = nullptr;

void UActorBrowserConfig::Initialize()
{
	if(!Instance)
	{
		Instance = NewObject<UActorBrowserConfig>(); 
		Instance->AddToRoot();
	}
}
