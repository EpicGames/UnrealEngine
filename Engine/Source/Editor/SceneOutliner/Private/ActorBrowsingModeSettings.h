// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "EditorConfigBase.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "ActorBrowsingModeSettings.generated.h"

class FName;
class UObject;

UENUM()
enum class EActorBrowsingFolderDoubleClickMethod : uint8
{
	/** Expands / Collapses the folder */
	ToggleExpansion,

	/** Marks / unmarks the folder as the Current Folder */
	ToggleCurrentFolder,
};

USTRUCT()
struct FActorBrowsingModeConfig
{
	GENERATED_BODY()

public:

	/** True when the Scene Outliner is hiding temporary/run-time Actors */
	UPROPERTY()
	bool bHideTemporaryActors = false;

	/** True when the Scene Outliner is showing only Actors that exist in the current level */
	UPROPERTY()
	bool bShowOnlyActorsInCurrentLevel = false;

	/** True when the Scene Outliner is showing only Actors that exist in the current data layers */
	UPROPERTY()
	bool bShowOnlyActorsInCurrentDataLayers = false;

	/** True when the Scene Outliner is showing only Actors that exist in the current content bundle */
	UPROPERTY()
	bool bShowOnlyActorsInCurrentContentBundle = false;

	/** True when the Scene Outliner is only displaying selected Actors */
	UPROPERTY()
	bool bShowOnlySelectedActors = false;

	/** True when the Scene Outliner is not displaying Actor Components*/
	UPROPERTY()
	bool bHideActorComponents = true;

	/** True when the Scene Outliner is not displaying LevelInstances */
	UPROPERTY()
	bool bHideLevelInstanceHierarchy = false;

	/** True when the Scene Outliner is not displaying unloaded actors */
	UPROPERTY()
	bool bHideUnloadedActors = false;

	/** True when the Scene Outliner is not displaying empty folders */
	UPROPERTY()
	bool bHideEmptyFolders = false;

	/** True when the Scene Outliner updates when an actor is selected in the viewport */
	UPROPERTY()
	bool bAlwaysFrameSelection = true;

	/** Specifies the behavior of double click on a folder */
	UPROPERTY()
	EActorBrowsingFolderDoubleClickMethod FolderDoubleClickMethod = EActorBrowsingFolderDoubleClickMethod::ToggleExpansion;

	/** True if we want to allow updating content when in PIE with the PIE viewport focused */
	UPROPERTY()
	bool bShouldUpdateContentWhileInPIEFocused = false;

	/** True if we want to collapse Outliner tree on new selection, except for the item that was just selected */
	UPROPERTY()
	bool bCollapseOutlinerTreeOnNewSelection = false;
};

UCLASS(EditorConfig="ActorBrowsingMode")
class UActorBrowserConfig : public UEditorConfigBase
{
	GENERATED_BODY()
	
public:

	static void Initialize();
	static UActorBrowserConfig* Get() { return Instance; }

	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FActorBrowsingModeConfig> ActorBrowsers;

private:

	static TObjectPtr<UActorBrowserConfig> Instance;
};