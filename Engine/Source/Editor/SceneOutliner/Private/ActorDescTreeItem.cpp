// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorDescTreeItem.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterPinnedActors.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerDragDrop.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "ActorEditorUtils.h"
#include "ClassIconFinder.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"
#include "Logging/MessageLog.h"
#include "SSocketChooser.h"
#include "ToolMenus.h"
#include "LevelEditorViewport.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "SceneOutliner_ActorDescTreeItem"

const FSceneOutlinerTreeItemType FActorDescTreeItem::Type(&IActorBaseTreeItem::Type);

template<>
struct TWidgetTypeTraits<struct SActorDescTreeLabel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

struct SActorDescTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SActorDescTreeLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FActorDescTreeItem& ActorDescItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());

		TreeItemPtr = StaticCastSharedRef<FActorDescTreeItem>(ActorDescItem.AsShared());

		HighlightText = SceneOutliner.GetFilterHighlightText();

		TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;

		auto MainContent = SNew(SHorizontalBox)
			// Main actor desc label
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
				.Text(this, &SActorDescTreeLabel::GetDisplayText)
				.ToolTipText(this, &SActorDescTreeLabel::GetTooltipText)
				.HighlightText(HighlightText)
				.ColorAndOpacity(this, &SActorDescTreeLabel::GetForegroundColorInternal)
				.OnTextCommitted(this, &SActorDescTreeLabel::OnLabelCommitted)
				.OnVerifyTextChanged(this, &SActorDescTreeLabel::OnVerifyItemLabelChanged)
				.IsSelected(FIsSelected::CreateSP(&InRow, &STableRow<FSceneOutlinerTreeItemPtr>::IsSelectedExclusively))
				.IsReadOnly_Lambda([Item = ActorDescItem.AsShared(), this]()
			{
				return !CanExecuteRenameRequest(Item.Get());
			})
			]

		+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f, 0.f, 3.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(this, &SActorDescTreeLabel::GetTypeText)
				.Visibility(this, &SActorDescTreeLabel::GetTypeTextVisibility)
				.HighlightText(HighlightText)
			];

		if (WeakSceneOutliner.Pin()->GetMode()->IsInteractive())
		{
			ActorDescItem.RenameRequestEvent.BindSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
		}

		ChildSlot
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
				[
					SNew(SBox)
					.WidthOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
					.HeightOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
					[
						SNew(SImage)
						.Image(this, &SActorDescTreeLabel::GetIcon)
					.ToolTipText(this, &SActorDescTreeLabel::GetIconTooltip)
					.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]

			+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f)
				[
					MainContent
				]
			];
		SetForegroundColor(TAttribute<FSlateColor>::CreateRaw(this, &SActorDescTreeLabel::GetForegroundColorInternal));
	}

private:
	TWeakPtr<FActorDescTreeItem> TreeItemPtr;
	TAttribute<FText> HighlightText;

	FText GetDisplayText() const
	{
		if (TSharedPtr<FActorDescTreeItem> TreeItem = TreeItemPtr.Pin())
		{
			if (FWorldPartitionActorDescInstance* ActorDescInstance = *TreeItem->ActorDescHandle)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("ActorLabel"), FText::FromString(TreeItem->GetDisplayString()));
				Args.Add(TEXT("UnloadState"), ActorDescInstance->GetUnloadedReason());

				return FText::Format(LOCTEXT("UnloadedActorDisplay", "{ActorLabel} ({UnloadState})"), Args);
			}
		}
		return FText();
	}

	FText GetTooltipText() const
	{
		return FText();
	}

	FText GetTypeText() const
	{
		if (TSharedPtr<FActorDescTreeItem> TreeItem = TreeItemPtr.Pin())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *TreeItem->ActorDescHandle)
			{
				return FText::FromName(ActorDescInstance->GetDisplayClassName());
			}
		}

		return FText();
	}

	EVisibility GetTypeTextVisibility() const
	{
		return HighlightText.Get().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
	}

	const FSlateBrush* GetIcon() const
	{
		TSharedPtr<FActorDescTreeItem> TreeItem = TreeItemPtr.Pin();

		if (TreeItem.IsValid() && WeakSceneOutliner.IsValid())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *TreeItem->ActorDescHandle)
			{
				const FName IconName = ActorDescInstance->GetDisplayClassName();

				const FSlateBrush* CachedBrush = WeakSceneOutliner.Pin()->GetCachedIconForClass(IconName);
				if (CachedBrush != nullptr)
				{
					return CachedBrush;
				}
				else if (IconName != NAME_None)
				{
					const FSlateBrush* FoundSlateBrush = FSlateIconFinder::FindIconForClass(ActorDescInstance->GetActorNativeClass()).GetIcon();
					WeakSceneOutliner.Pin()->CacheIconForClass(IconName, FoundSlateBrush);
					return FoundSlateBrush;
				}
			}
		}

		return nullptr;
	}

	const FSlateBrush* GetIconOverlay() const
	{
		return nullptr;
	}

	FText GetIconTooltip() const
	{
		return FText();
	}

	/**
	 * Returns the Foreground Color of this widget.
	 * Used to avoid overriding SCompoundWidget::GetForegroundColor for invalidation to work properly.
	 */
	FSlateColor GetForegroundColorInternal() const
	{
		if (const auto TreeItem = TreeItemPtr.Pin())
		{
			if (auto BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItem))
			{
				return BaseColor.GetValue();
			}
			
			if(WeakSceneOutliner.IsValid())
			{
				// Use the normal foreground color for selected items to make them readable
				if(WeakSceneOutliner.Pin()->GetTree().IsItemSelected(TreeItem))
				{
					return FSlateColor::UseSubduedForeground();
				}
			}
		}
		
		return FSceneOutlinerCommonLabelData::DarkColor;
	}

	bool OnVerifyItemLabelChanged(const FText&, FText&)
	{
		// don't allow label change for unloaded actor items
		return false;
	}

	void OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
	{
		// not supported.
	}
};

FActorDescTreeItem::FActorDescTreeItem(const FGuid& InActorGuid, UActorDescContainerInstance* InContainerInstance)
	: IActorBaseTreeItem(Type)
	, ActorDescHandle(InContainerInstance, InActorGuid)
	, ID(ComputeTreeItemID(InActorGuid, InContainerInstance))
	, ActorGuid(InActorGuid)
{
	if (const FWorldPartitionActorDescInstance* const ActorDescInstance = *ActorDescHandle)
	{
		DisplayString = ActorDescInstance->GetActorLabelString();
	}
	else
	{
		DisplayString = LOCTEXT("ActorLabelForMissingActor", "(Deleted Actor)").ToString();
	}

	Flags.bIsExpanded = false;
}

FActorDescTreeItem::FActorDescTreeItem(const FWorldPartitionActorDescInstance* InActorDescInstance)
	: IActorBaseTreeItem(Type)
	, ActorDescHandle(InActorDescInstance->GetContainerInstance(), InActorDescInstance->GetGuid())
	, ID(ComputeTreeItemID(InActorDescInstance->GetGuid(), InActorDescInstance->GetContainerInstance()))
	, ActorGuid(InActorDescInstance->GetGuid())
{
	if (const FWorldPartitionActorDescInstance* const ActorDescInstance = *ActorDescHandle)
	{
		DisplayString = ActorDescInstance->GetActorLabelString();
	}
	else
	{
		DisplayString = LOCTEXT("ActorLabelForMissingActor", "(Deleted Actor)").ToString();
	}

	Flags.bIsExpanded = false;
}

FSceneOutlinerTreeItemID FActorDescTreeItem::ComputeTreeItemID(FGuid InActorGuid, UActorDescContainerInstance* InContainerInstance)
{
	return FSceneOutlinerTreeItemID(FGuid::Combine(InActorGuid, InContainerInstance->GetContainerActorGuid()));
}

bool FActorDescTreeItem::ShouldDisplayInOutliner(const FWorldPartitionActorDescInstance* InActorDescInstance)
{
	return InActorDescInstance && InActorDescInstance->IsListedInSceneOutliner() && (InActorDescInstance->GetActorIsRuntimeOnly() || InActorDescInstance->IsEditorRelevant());
}

FSceneOutlinerTreeItemID FActorDescTreeItem::GetID() const
{
	return ID;
}

FString FActorDescTreeItem::GetDisplayString() const
{
	return DisplayString;
}

bool FActorDescTreeItem::CanInteract() const
{
	return ActorDescHandle.IsValid();
}

TSharedRef<SWidget> FActorDescTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SActorDescTreeLabel, *this, Outliner, InRow);
}

void FActorDescTreeItem::FocusActorBounds() const
{
	if (FWorldPartitionActorDescInstance const* ActorDescInstance = *ActorDescHandle)
	{
		const bool bActiveViewportOnly = true;
		const FBox EditorBounds = ActorDescInstance->GetEditorBounds();
		if (EditorBounds.IsValid)
		{
			GEditor->MoveViewportCamerasToBox(EditorBounds, bActiveViewportOnly, 0.5f);
		}
	}
}

void FActorDescTreeItem::CopyActorFilePathtoClipboard() const
{
	if (FWorldPartitionActorDescInstance const* ActorDescInstance = *ActorDescHandle)
	{
		FString PackageFilename;
		if (FPackageName::TryConvertLongPackageNameToFilename(ActorDescInstance->GetActorPackage().ToString(), PackageFilename, FPackageName::GetAssetPackageExtension()))
		{
			FString Result = FPaths::ConvertRelativePathToFull(PackageFilename);
			FPaths::MakePlatformFilename(Result);
			FPlatformApplicationMisc::ClipboardCopy(*Result);
		}
	}
}

void FActorDescTreeItem::GenerateContextMenu(UToolMenu* Menu, SSceneOutliner&)
{
	FToolMenuSection& Section = Menu->AddSection("Section");
	Section.AddMenuEntry("FocusActorBounds", LOCTEXT("FocusActorBounds", "Focus Actor Bounds"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &FActorDescTreeItem::FocusActorBounds)));
	Section.AddMenuEntry("CopyActorFilePath", LOCTEXT("CopyActorFilePath", "Copy Actor File Path"), LOCTEXT("CopyActorFilePathTooltip", "Copy the file path where this actor is saved"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"), FUIAction(FExecuteAction::CreateSP(this, &FActorDescTreeItem::CopyActorFilePathtoClipboard)));
}

bool FActorDescTreeItem::GetVisibility() const
{
	return false;
}

bool FActorDescTreeItem::ShouldShowPinnedState() const
{
	return FLoaderAdapterPinnedActors::SupportsPinning(*ActorDescHandle);
}

bool FActorDescTreeItem::GetPinnedState() const
{
	if (ActorDescHandle.IsValid() && ActorDescHandle->GetContainerInstance())
	{
		UWorldPartition* WorldPartition = ActorDescHandle->GetContainerInstance()->GetOuterWorldPartition();
		return WorldPartition ? WorldPartition->IsActorPinned(GetGuid()) : false;
	}
	return false;
}

FString FActorDescTreeItem::GetPackageName() const
{
	if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescHandle)
	{
		return ActorDescInstance->GetActorPackage().ToString();
	}
	
	return IActorBaseTreeItem::GetPackageName();
}

UExternalDataLayerAsset* FActorDescTreeItem::GetExternalDataLayerAsset() const
{
	if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescHandle)
	{
		const FSoftObjectPath& ExternalDataLayerAsset = ActorDescInstance->GetExternalDataLayerAsset();
		if (CachedExternalDataLayerAsset.ToSoftObjectPath() != ExternalDataLayerAsset)
		{
			CachedExternalDataLayerAsset = ExternalDataLayerAsset;
		}
		return CachedExternalDataLayerAsset.Get();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
