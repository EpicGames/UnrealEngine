// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolderHierarchy.h"
#include "Engine/Level.h"
#include "ISceneOutlinerMode.h"
#include "WorldTreeItem.h"
#include "LevelTreeItem.h"
#include "ActorTreeItem.h"
#include "ActorFolderTreeItem.h"
#include "EditorActorFolders.h"
#include "EditorFolderUtils.h"
#include "ActorMode.h"
#include "LevelInstance/LevelInstanceInterface.h"

FActorFolderHierarchy::FActorFolderHierarchy(ISceneOutlinerMode* InMode, const TWeakObjectPtr<UWorld>& World, const FFolder::FRootObject& InRootObject)
	: ISceneOutlinerHierarchy(InMode)
	, RepresentingWorld(World)
	, RootObject(InRootObject)
{
	// ActorFolderHierarchy should only be used with a mode which is showing folders
	check(Mode->ShouldShowFolders());
}

FSceneOutlinerTreeItemPtr FActorFolderHierarchy::FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate)
{
	if (Item.IsA<FWorldTreeItem>())
	{
		return nullptr;
	}
	else if (const FActorFolderTreeItem* ActorFolderItem = Item.CastTo<FActorFolderTreeItem>())
	{
		const FFolder ParentPath = ActorFolderItem->GetFolder().GetParent();

		// If the folder has no parent path
		if (ParentPath.IsNone())
		{
			if (UObject* Object = ParentPath.GetRootObjectPtr())
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(Object))
				{
					return *ParentItem;
				}
				else if (bCreate)
				{
					FFolder Folder = ActorFolderItem->GetFolder();
					if (Folder.GetRootObject() == RootObject)
					{
						// If item belongs to a LevelInstance
						if (ILevelInstanceInterface* RootLevelInstance = Cast<ILevelInstanceInterface>(Folder.GetRootObjectPtr()))
						{
							return Mode->CreateItemFor<FActorTreeItem>(CastChecked<AActor>(RootLevelInstance), true);
						}
					}
				}
			}
			else
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ActorFolderItem->World.Get()))
				{
					return *ParentItem;
				}
				else if (bCreate)
				{
					// Parent World
					UWorld* OwningWorld = ActorFolderItem->World.Get();
					check(OwningWorld);
					return Mode->CreateItemFor<FWorldTreeItem>(OwningWorld, true);
				}
			}
		}
		else if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentPath))
		{
			return *ParentItem;
		}
		else if(bCreate)
		{
			return Mode->CreateItemFor<FActorFolderTreeItem>(FActorFolderTreeItem(ParentPath, ActorFolderItem->World), true);
		}
	}

	return nullptr;
}

void FActorFolderHierarchy::CreateWorldChildren(UWorld* World, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	FActorFolders::Get().ForEachFolderWithRootObject(*World, RootObject, [this, World, &OutItems](const FFolder& Folder)
	{
		if (FSceneOutlinerTreeItemPtr FolderItem = Mode->CreateItemFor<FActorFolderTreeItem>(FActorFolderTreeItem(Folder, World)))
		{
			OutItems.Add(FolderItem);
		}
		return true;
	});

	if (FFolder::IsRootObjectValid(RootObject))
	{
		UObject* RootObjectPtr = FFolder::GetRootObjectPtr(RootObject);
		if (ILevelInstanceInterface* RootLevelInstance = Cast<ILevelInstanceInterface>(RootObjectPtr))
		{
			if (FSceneOutlinerTreeItemPtr ActorItem = Mode->CreateItemFor<FActorTreeItem>(CastChecked<AActor>(RootLevelInstance), true))
			{
				OutItems.Add(ActorItem);
			}
		}
		else if (ULevel* RootLevel = Cast<ULevel>(RootObjectPtr))
		{
			if (FActorMode::IsActorLevelDisplayable(RootLevel))
			{
				if (FSceneOutlinerTreeItemPtr ActorItem = Mode->CreateItemFor<FLevelTreeItem>(RootLevel, true))
				{
					OutItems.Add(ActorItem);
				}
			}
		}
	}
}

void FActorFolderHierarchy::CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	check(RepresentingWorld.IsValid());

	if (FFolder::IsRootObjectPersistentLevel(RootObject))
	{
		if (FSceneOutlinerTreeItemPtr WorldItem = Mode->CreateItemFor<FWorldTreeItem>(RepresentingWorld.Get()))
		{
			OutItems.Add(WorldItem);
		}
	}

	CreateWorldChildren(RepresentingWorld.Get(), OutItems);
}

void FActorFolderHierarchy::CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const
{
	auto CreateChildrenFolders = [this](UWorld* InWorld, const FFolder& InParentFolder, const FFolder::FRootObject& InFolderRootObject, TArray<FSceneOutlinerTreeItemPtr>& OutChildren)
	{
		FActorFolders::Get().ForEachFolderWithRootObject(*InWorld, InFolderRootObject, [this, InWorld, &InParentFolder, &OutChildren](const FFolder& Folder)
		{
			if (Folder.IsChildOf(InParentFolder))
			{
				if (FSceneOutlinerTreeItemPtr NewFolderItem = Mode->CreateItemFor<FActorFolderTreeItem>(FActorFolderTreeItem(Folder, InWorld)))
				{
					OutChildren.Add(NewFolderItem);
				}
			}
			return true;
		});
	};

	UWorld* World = RepresentingWorld.Get();
	if (const FWorldTreeItem* WorldItem = Item->CastTo<FWorldTreeItem>())
	{
		check(WorldItem->World == RepresentingWorld);
		CreateWorldChildren(WorldItem->World.Get(), OutChildren);
	}
	else if (FActorTreeItem* ParentActorItem = Item->CastTo<FActorTreeItem>())
	{
		AActor* ParentActor = ParentActorItem->Actor.Get();
		if (const ILevelInstanceInterface* LevelInstanceParent = Cast<ILevelInstanceInterface>(ParentActor))
		{
			check(ParentActor->GetWorld() == World);
			FFolder ParentFolder = ParentActor->GetFolder();
			CreateChildrenFolders(World, ParentFolder, ParentActor, OutChildren);
		}
	}
	else if (FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>())
	{
		check(FolderItem->World.Get() == World);
		FFolder ParentFolder = FolderItem->GetFolder();
		check(!ParentFolder.IsNone());
		CreateChildrenFolders(World, ParentFolder, ParentFolder.GetRootObject(), OutChildren);
	}
}

