// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolderPickingMode.h"
#include "ActorFolderTreeItem.h"
#include "ActorFolderHierarchy.h"

#define LOCTEXT_NAMESPACE "SceneOutliner_ActoFolderPickingMode"

FActorFolderPickingMode::FActorFolderPickingMode(SSceneOutliner* InSceneOutliner, FOnSceneOutlinerItemPicked InOnItemPicked, TWeakObjectPtr<UWorld> InSpecifiedWorldToDisplay, const FFolder::FRootObject& InRootObject)
	: FActorMode(FActorModeParams(InSceneOutliner, InSpecifiedWorldToDisplay))
	, OnItemPicked(InOnItemPicked)
	, RootObject(InRootObject)
{
	Rebuild();
}

TUniquePtr<ISceneOutlinerHierarchy> FActorFolderPickingMode::CreateHierarchy()
{
	return MakeUnique<FActorFolderHierarchy>(this, RepresentingWorld, RootObject);
}

void FActorFolderPickingMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	// fire off the notification to whoever is listening.
	// This may often cause the widget itself to be enqueued for destruction
	auto SelectedItems = SceneOutliner->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		auto FirstItem = SelectedItems[0];
		if (FirstItem->CanInteract())
		{
			OnItemPicked.ExecuteIfBound(FirstItem.ToSharedRef());
		}
	}
}

void FActorFolderPickingMode::OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType)
{
	// In folder picking mode, we check to see if we have a selected folder, and if so, fire
	// off the notification to whoever is listening.  This may often cause the widget itself
	// to be enqueued for destruction
	TArray<FActorFolderTreeItem*> Folders;
	Selection.Get(Folders);
	if (Folders.Num() == 1 && Folders[0])
	{
		// Signal that a folder was selected. We assume it is valid as it wouldn't not have been added to the selection if not
		SceneOutliner->SetItemSelection(Folders[0]->AsShared(), true, ESelectInfo::OnKeyPress);
	}
}

void FActorFolderPickingMode::CreateViewContent(FMenuBuilder& MenuBuilder)
{
	FActorMode::BuildWorldPickerMenu(MenuBuilder);
}

#undef LOCTEXT_NAMESPACE