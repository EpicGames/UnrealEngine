// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolderTreeItem.h"
#include "ActorTreeItem.h"
#include "SSceneOutliner.h"
#include "ActorEditorUtils.h"
#include "EditorActorFolders.h"
#include "EditorFolderUtils.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "ActorDescTreeItem.h"
#include "Engine/World.h"
#include "WorldPersistentFolders.h"
#include "Engine/Level.h"
#include "LevelUtils.h"
#include "SceneOutlinerHelpers.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"

#define LOCTEXT_NAMESPACE "SceneOutliner_ActorFolderTreeItem"

const FSceneOutlinerTreeItemType FActorFolderTreeItem::Type(&FFolderTreeItem::Type);

template<>
struct TWidgetTypeTraits<struct SActorFolderTreeLabel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

struct SActorFolderTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SActorFolderTreeLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FActorFolderTreeItem& FolderItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		TreeItemPtr = StaticCastSharedRef<FActorFolderTreeItem>(FolderItem.AsShared());
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());

		TSharedPtr<SInlineEditableTextBlock> InlineTextBlock = SNew(SInlineEditableTextBlock)
			.Text(this, &SActorFolderTreeLabel::GetDisplayText)
			.ToolTipText(this, &SActorFolderTreeLabel::GetTooltipText)
			.HighlightText(SceneOutliner.GetFilterHighlightText())
			.ColorAndOpacity(this, &SActorFolderTreeLabel::GetForegroundColorInternal)
			.OnTextCommitted(this, &SActorFolderTreeLabel::OnLabelCommitted)
			.OnVerifyTextChanged(this, &SActorFolderTreeLabel::OnVerifyItemLabelChanged)
			.OnEnterEditingMode(this, &SActorFolderTreeLabel::OnEnterEditingMode)
			.OnExitEditingMode(this, &SActorFolderTreeLabel::OnExitEditingMode)
			.IsSelected(FIsSelected::CreateSP(&InRow, &STableRow<FSceneOutlinerTreeItemPtr>::IsSelectedExclusively))
			.IsReadOnly_Lambda([Item = FolderItem.AsShared(), this]()
		{
			return !CanExecuteRenameRequest(Item.Get());
		});

		if (WeakSceneOutliner.Pin()->GetMode()->IsInteractive())
		{
			FolderItem.RenameRequestEvent.BindSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
		}

		ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
				[
					SNew(SBox)
					.WidthOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
					.HeightOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
					[
						SNew(SImage)
						.Image(this, &SActorFolderTreeLabel::GetIcon)
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 2.0f)
				[
					InlineTextBlock.ToSharedRef()
				]
			];
		SetForegroundColor(TAttribute<FSlateColor>::CreateRaw(this, &SActorFolderTreeLabel::GetForegroundColorInternal));
	}

private:
	TWeakPtr<FActorFolderTreeItem> TreeItemPtr;

	bool IsInActorEditorContext() const
	{
		auto Folder = TreeItemPtr.Pin();

		if(Folder->IsValid())
		{
			return SceneOutliner::FSceneOutlinerHelpers::IsFolderCurrent(Folder->GetFolder(), Folder->World.Get());
		}

		return false;
	}

	FText GetDisplayText() const
	{
		auto Folder = TreeItemPtr.Pin();
		if (Folder.IsValid())
		{
			if (!bInEditingMode)
			{
				FText IsCurrentSuffixText = IsInActorEditorContext() ? FText(LOCTEXT("IsCurrentSuffix", " (Current)")) : FText::GetEmpty();
				return FText::Format(LOCTEXT("LevelInstanceDisplay", "{0}{1}"), FText::FromString(Folder->GetDisplayString()), IsCurrentSuffixText);
			}
			return FText::FromString(Folder->GetDisplayString());
		}
		return FText();
	}

	FText GetTooltipText() const
	{
		if (const FSceneOutlinerTreeItemPtr TreeItem = TreeItemPtr.Pin())
		{
			FText Description = IsInActorEditorContext() ? LOCTEXT("ActorFolderIsCurrentDescription", "\nThis is your current folder. New actors you create will appear here.") : FText::GetEmpty();
			return FText::Format(LOCTEXT("DataLayerTooltipText", "{0}{1}"), FText::FromString(TreeItem->GetDisplayString()), Description);
		}

		return FText();
	}

	const FSlateBrush* GetIcon() const
	{
		auto TreeItem = TreeItemPtr.Pin();
		if (!TreeItem.IsValid())
		{
			return FAppStyle::Get().GetBrush(TEXT("SceneOutliner.FolderClosed"));
		}

		if (TreeItem->Flags.bIsExpanded && TreeItem->GetChildren().Num())
		{
			return FAppStyle::Get().GetBrush(TEXT("SceneOutliner.FolderOpen"));
		}
		else
		{
			return FAppStyle::Get().GetBrush(TEXT("SceneOutliner.FolderClosed"));
		}
	}

	/**
	 * Returns the Foreground Color of this widget.
	 * Used to avoid overriding SCompoundWidget::GetForegroundColor for invalidation to work properly.
	 */
	FSlateColor GetForegroundColorInternal() const
	{
		if (auto BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItemPtr.Pin()))
		{
			return BaseColor.GetValue();
		}

		if (IsInActorEditorContext())
		{
			return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
		}

		return FSlateColor::UseForeground();
	}

	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
	{
		auto TreeItem = TreeItemPtr.Pin();
		if (!TreeItem.IsValid())
		{
			OutErrorMessage = LOCTEXT("RenameFailed_TreeItemDeleted", "Tree item no longer exists");
			return false;
		}

		return SceneOutliner::FSceneOutlinerHelpers::ValidateFolderName(TreeItem->GetFolder(), TreeItem->World.Get(), InLabel, OutErrorMessage);
	}

	void OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
	{
		auto TreeItem = TreeItemPtr.Pin();
		if (TreeItem.IsValid())
		{
			const FText TrimmedLabel = FText::TrimPrecedingAndTrailing(InLabel);
			if (!TrimmedLabel.ToString().Equals(TreeItem->GetLeafName().ToString(), ESearchCase::CaseSensitive))
			{
				// Rename the item
				SceneOutliner::FSceneOutlinerHelpers::RenameFolder(TreeItem->GetFolder(), TrimmedLabel, TreeItem->World.Get());

				auto Outliner = WeakSceneOutliner.Pin();
				if (Outliner.IsValid())
				{
					Outliner->SetKeyboardFocus();
				}
			}
		}
	}

	void OnEnterEditingMode()
	{
		bInEditingMode = true;
	}

	void OnExitEditingMode()
	{
		bInEditingMode = false;
	}

	bool bInEditingMode = false;
};

FActorFolderTreeItem::FActorFolderTreeItem(const FFolder& InFolder, const TWeakObjectPtr<UWorld>& InWorld)
	: FFolderTreeItem(InFolder, Type)
	, World(InWorld)
{
	FFolderTreeItem::SetPath(InFolder.GetPath());
	// Initialize ActorFolder
	ActorFolder = InFolder.GetActorFolder();

	if (World.IsValid())
	{
		Flags.bIsExpanded = !World->IsGameWorld() ? FActorFolders::Get().IsFolderExpanded(*World, GetFolder()) : true;
	}
}

void FActorFolderTreeItem::OnExpansionChanged()
{
	if (!World.IsValid())
	{
		return;
	}

	// Update the central store of folder properties with this folder's new expansion state
	FActorFolders::Get().SetIsFolderExpanded(*World, GetFolder(), Flags.bIsExpanded);
}

void FActorFolderTreeItem::Delete(const FFolder& InNewParentFolder)
{
	if (!World.IsValid())
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("DeleteFolderTransaction", "Delete Folder"));

	const FFolder::FRootObject& NewParentRootObject = InNewParentFolder.GetRootObject();

	for (const TWeakPtr<ISceneOutlinerTreeItem>& ChildPtr : GetChildren())
	{
		FSceneOutlinerTreeItemPtr Child = ChildPtr.Pin();
		if (const FActorTreeItem* ActorItem = Child->CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				check(Actor->GetFolderRootObject() == NewParentRootObject);
				// When using actor folders, no need to update actors since the folder path resolving is dynamic
				if (!Actor->GetLevel()->IsUsingActorFolders())
				{
					Actor->SetFolderPath_Recursively(InNewParentFolder.GetPath());
				}
			}
		}
		else if (FActorFolderTreeItem* FolderItem = Child->CastTo<FActorFolderTreeItem>())
		{
			// When using actor folders, no need to update child folders since the folder path resolving is dynamic
			if (!FolderItem->GetActorFolder())
			{
				FolderItem->MoveTo(InNewParentFolder);
			}
		}
	}

	FActorFolders::Get().DeleteFolder(*World, GetFolder());

	// Remove this folder from parent otherwise when parent folder and child folder deleted together, parent will call MoveTo on deleted child folder leading to crash
	if (FSceneOutlinerTreeItemPtr ParentPtr = GetParent())
	{
		ParentPtr->RemoveChild(SharedThis(this));
	}
}

void FActorFolderTreeItem::MoveTo(const FFolder& InNewParentFolder)
{
	if (World.IsValid())
	{
		check(InNewParentFolder.GetRootObject() == GetRootObject());
		// Get unique name
		const FFolder NewPath = FActorFolders::Get().GetFolderName(*World, InNewParentFolder, GetLeafName());
		FActorFolders::Get().RenameFolderInWorld(*World, GetFolder(), NewPath);
	}
}

void FActorFolderTreeItem::SetPath(const FName& InNewPath)
{
	FFolderTreeItem::SetPath(InNewPath);

	// Validate that if the item's ActorFolder object is set, that changing the path keeps the same ActorFolder but with a different path
	check(!ActorFolder.IsValid() || (ActorFolder->GetFolder().GetPath() == Path));
}

void FActorFolderTreeItem::CreateSubFolder(TWeakPtr<SSceneOutliner> WeakOutliner)
{
	auto Outliner = WeakOutliner.Pin();

	if (Outliner.IsValid() && World.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));

		const FFolder NewFolderName = FActorFolders::Get().GetDefaultFolderName(*World, GetFolder());
		FActorFolders::Get().CreateFolder(*World, NewFolderName);

		// At this point the new folder will be in our newly added list, so select it and open a rename when it gets refreshed
		Outliner->OnItemAdded(NewFolderName, SceneOutliner::ENewItemAction::Select | SceneOutliner::ENewItemAction::Rename);
	}
}

TSharedRef<SWidget> FActorFolderTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SActorFolderTreeLabel, *this, Outliner, InRow);
}

bool FActorFolderTreeItem::CanChangeChildrenPinnedState() const
{
	if (World.IsValid() && !World->IsGameWorld() && World->IsPartitionedWorld())
	{
		return FFolder::IsRootObjectPersistentLevel(GetRootObject());
	}
	return false;
}

bool FActorFolderTreeItem::ShouldShowVisibilityState() const
{
	// Visibility state can only be shown when folder is part of the persistent level (PIE or Editor) or
	// if it is an actual sub-level that is not instanced (sublevels prior to level instances or editing level instance)
	ULevel* Level = FFolder::GetRootObjectAssociatedLevel(GetRootObject());
	return World.IsValid() && Level && (Level->IsPersistentLevel() || !Level->IsInstancedLevel());
}

bool FActorFolderTreeItem::ShouldRemoveOnceLastChildRemoved() const
{
	if (FFolderTreeItem::ShouldRemoveOnceLastChildRemoved())
	{
		return true;
	}
	// Don't show empty folders in game world
	const bool bIsGameWorld = World.IsValid() && World->IsGameWorld();
	return bIsGameWorld;
}

FFolder FActorFolderTreeItem::GetFolder() const
{
	// Use Folder resolved by ActorFolder when valid (a null owning world would will fail resolving, see UActorFolder::GetFolder())
	FFolder Folder = ActorFolder.IsValid() ? ActorFolder->GetFolder() : FFolder::GetInvalidFolder();
	return Folder.IsValid() ? Folder : FFolderTreeItem::GetFolder();
}

FString FActorFolderTreeItem::GetPackageName() const
{
	if (const UActorFolder* ActorFolderPtr = GetActorFolder())
	{
		if (ActorFolderPtr->IsPackageExternal())
		{
			return ActorFolderPtr->GetExternalPackage()->GetName();
		}
	}
	
	return FFolderTreeItem::GetPackageName();
}

bool FActorFolderTreeItem::CanInteract() const
{
	if (!FFolderTreeItem::CanInteract())
	{
		return false;
	}

	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = World.IsValid() ? World->GetSubsystem<ULevelInstanceSubsystem>() : nullptr)
	{
		if (ILevelInstanceInterface* EditingLevelInstance = LevelInstanceSubsystem->GetEditingLevelInstance())
		{
			if (GetRootObject() != FFolder::FRootObject(CastChecked<AActor>(EditingLevelInstance)))
			{
				return false;
			}
		}
	}

	ULevel* Level = ActorFolder.IsValid() ? ActorFolder->GetOuterULevel() : nullptr;
	return Level ? !FLevelUtils::IsLevelLocked(Level) : true;
}

#undef LOCTEXT_NAMESPACE
