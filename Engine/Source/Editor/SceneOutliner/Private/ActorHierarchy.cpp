// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorHierarchy.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "WorldTreeItem.h"
#include "LevelTreeItem.h"
#include "ActorTreeItem.h"
#include "ActorDescTreeItem.h"
#include "ComponentTreeItem.h"
#include "ActorFolderTreeItem.h"
#include "ISceneOutlinerMode.h"
#include "ActorEditorUtils.h"
#include "LevelUtils.h"
#include "GameFramework/WorldSettings.h"
#include "EditorActorFolders.h"
#include "EditorFolderUtils.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Modules/ModuleManager.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "ActorFolder.h"
#include "ActorMode.h"
#include "SSceneOutliner.h"

TUniquePtr<FActorHierarchy> FActorHierarchy::Create(ISceneOutlinerMode* Mode, const TWeakObjectPtr<UWorld>& World)
{
	FActorHierarchy* Hierarchy = new FActorHierarchy(Mode, World);

	Create_Internal(Hierarchy, World);

	return TUniquePtr<FActorHierarchy>(Hierarchy);
}

void FActorHierarchy::Create_Internal(FActorHierarchy* Hierarchy, const TWeakObjectPtr<UWorld>& World)
{
	GEngine->OnLevelActorAdded().AddRaw(Hierarchy, &FActorHierarchy::OnLevelActorAdded);
	GEngine->OnLevelActorDeleted().AddRaw(Hierarchy, &FActorHierarchy::OnLevelActorDeleted);
	GEngine->OnLevelActorDetached().AddRaw(Hierarchy, &FActorHierarchy::OnLevelActorDetached);
	GEngine->OnLevelActorAttached().AddRaw(Hierarchy, &FActorHierarchy::OnLevelActorAttached);
	GEngine->OnLevelActorFolderChanged().AddRaw(Hierarchy, &FActorHierarchy::OnLevelActorFolderChanged);
	GEngine->OnLevelActorListChanged().AddRaw(Hierarchy, &FActorHierarchy::OnLevelActorListChanged);
	GEngine->OnActorFolderAdded().AddRaw(Hierarchy, &FActorHierarchy::OnActorFolderAdded);
	GEngine->OnActorFoldersUpdatedEvent().AddRaw(Hierarchy, &FActorHierarchy::OnActorFoldersUpdatedEvent);

	IWorldPartitionEditorModule::Get().OnWorldPartitionCreated().AddRaw(Hierarchy, &FActorHierarchy::OnWorldPartitionCreated);

	if (World.IsValid())
	{
		if (World->PersistentLevel)
		{
			World->PersistentLevel->OnLoadedActorAddedToLevelEvent.AddRaw(Hierarchy, &FActorHierarchy::OnLoadedActorAdded);
			World->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.AddRaw(Hierarchy, &FActorHierarchy::OnLoadedActorRemoved);
		}

		World->OnWorldPartitionInitialized().AddRaw(Hierarchy, &FActorHierarchy::OnWorldPartitionInitialized);
		World->OnWorldPartitionUninitialized().AddRaw(Hierarchy, &FActorHierarchy::OnWorldPartitionUninitialized);

		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			WorldPartition->OnActorDescInstanceAddedEvent.AddRaw(Hierarchy, &FActorHierarchy::OnActorDescInstanceAdded);
			WorldPartition->OnActorDescInstanceRemovedEvent.AddRaw(Hierarchy, &FActorHierarchy::OnActorDescInstanceRemoved);
		}
	}

	FWorldDelegates::LevelAddedToWorld.AddRaw(Hierarchy, &FActorHierarchy::OnLevelAdded);
	FWorldDelegates::LevelRemovedFromWorld.AddRaw(Hierarchy, &FActorHierarchy::OnLevelRemoved);

	auto& Folders = FActorFolders::Get();
	Folders.OnFolderCreated.AddRaw(Hierarchy, &FActorHierarchy::OnBroadcastFolderCreate);
	Folders.OnFolderMoved.AddRaw(Hierarchy, &FActorHierarchy::OnBroadcastFolderMove);
	Folders.OnFolderDeleted.AddRaw(Hierarchy, &FActorHierarchy::OnBroadcastFolderDelete);
}

FActorHierarchy::FActorHierarchy(ISceneOutlinerMode* Mode, const TWeakObjectPtr<UWorld>& World)
	: ISceneOutlinerHierarchy(Mode)
	, RepresentingWorld(World)
{
}

FActorHierarchy::~FActorHierarchy()
{
	if (GEngine)
	{
		GEngine->OnLevelActorAdded().RemoveAll(this);
		GEngine->OnLevelActorDeleted().RemoveAll(this);
		GEngine->OnLevelActorDetached().RemoveAll(this);
		GEngine->OnLevelActorAttached().RemoveAll(this);
		GEngine->OnLevelActorFolderChanged().RemoveAll(this);
		GEngine->OnLevelActorListChanged().RemoveAll(this);
		GEngine->OnActorFolderAdded().RemoveAll(this);
		GEngine->OnActorFoldersUpdatedEvent().RemoveAll(this);
	}

	IWorldPartitionEditorModule::Get().OnWorldPartitionCreated().RemoveAll(this);

	if (RepresentingWorld.IsValid())
	{
		if (RepresentingWorld->PersistentLevel)
		{
			RepresentingWorld->PersistentLevel->OnLoadedActorAddedToLevelEvent.RemoveAll(this);
			RepresentingWorld->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.RemoveAll(this);
		}

		RepresentingWorld->OnWorldPartitionInitialized().RemoveAll(this);
		RepresentingWorld->OnWorldPartitionUninitialized().RemoveAll(this);

		if (UWorldPartition* WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			WorldPartition->OnActorDescInstanceAddedEvent.RemoveAll(this);
			WorldPartition->OnActorDescInstanceRemovedEvent.RemoveAll(this);
		}
	}

	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);

	auto& Folders = FActorFolders::Get();
	Folders.OnFolderCreated.RemoveAll(this);
	Folders.OnFolderMoved.RemoveAll(this);
	Folders.OnFolderDeleted.RemoveAll(this);
}

FSceneOutlinerTreeItemPtr FActorHierarchy::FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate)
{
	if (Item.IsA<FWorldTreeItem>())
	{
		return nullptr;
	}
	else if (const FActorTreeItem* ActorTreeItem = Item.CastTo<FActorTreeItem>())
	{
		if (AActor* Actor = ActorTreeItem->Actor.Get())
		{
			// Parent Actor (Actor attachement / parenting)
			if (AActor* ParentActor = Actor->GetSceneOutlinerParent())
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentActor))
				{
					return *ParentItem;
				}
				// If Parent can be listed in SceneOutliner return nullptr so it gets created
				else if (ParentActor->IsListedInSceneOutliner())
				{
					return bCreate ? CreateItemForActor(ParentActor, true) : nullptr;
				}
			}

			// Parent Folder
			FFolder Folder = Actor->GetFolder();
			if (Mode->ShouldShowFolders() && !Folder.IsNone())
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(Folder))
				{
					return *ParentItem;
				}
				else
				{
					return bCreate ? CreateItemForFolder(Folder, ActorTreeItem->Actor->GetWorld(), true) : nullptr;
				}
			}

			// Parent Level Instance
			if (ILevelInstanceInterface* OwningLevelInstance = Cast<ILevelInstanceInterface>(Folder.GetRootObjectPtr()))
			{
				const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor);
				const bool bIsAnEditingLevelInstance = LevelInstance ? (LevelInstance->IsEditing() || LevelInstance->IsEditingPropertyOverrides()) : false;
				// Parent this to a LevelInstance if the parent LevelInstance is being edited or if this is a sub LevelInstance which is being edited
				if (bShowingLevelInstances || OwningLevelInstance->IsEditing() || OwningLevelInstance->IsEditingPropertyOverrides() || bIsAnEditingLevelInstance)
				{
					AActor* OwningActor = CastChecked<AActor>(OwningLevelInstance);
					if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(OwningActor))
					{
						return *ParentItem;
					}
					else
					{
						return bCreate ? CreateItemForActor(OwningActor, true) : nullptr;
					}
				}
			}

			// Parent Level Using Actor Folders
			ULevel* OwningLevel = Cast<ULevel>(Folder.GetRootObjectPtr());
			// For the persistent level, fallback on the world
			if (FActorMode::IsActorLevelDisplayable(OwningLevel))
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(OwningLevel))
				{
					return *ParentItem;
				}
				else
				{
					return bCreate ? Mode->CreateItemFor<FLevelTreeItem>(OwningLevel, true) : nullptr;
				}
			}
		}
	}
	else if (const FActorFolderTreeItem* FolderItem = Item.CastTo<FActorFolderTreeItem>())
	{
		// We should never call FindParents on a folder item if folders are not being shown
		check(Mode->ShouldShowFolders());

		const FFolder ParentPath = FolderItem->GetFolder().GetParent();

		// Parent Folder
		if (!ParentPath.IsNone())
		{
			if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentPath))
			{
				return *ParentItem;
			}
			else
			{
				return bCreate ? CreateItemForFolder(ParentPath, FolderItem->World, true) : nullptr;
			}
		}
		// Parent Level Instance
		else if (ILevelInstanceInterface* OwningLevelInstance = Cast<ILevelInstanceInterface>(ParentPath.GetRootObjectPtr()))
		{
			if (bShowingLevelInstances || OwningLevelInstance->IsEditing() || OwningLevelInstance->IsEditingPropertyOverrides())
			{
				AActor* OwningActor = CastChecked<AActor>(OwningLevelInstance);
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(OwningActor))
				{
					return *ParentItem;
				}
				else
				{
					return bCreate ? CreateItemForActor(OwningActor, true) : nullptr;
				}
			}
		}
		// Parent Level Using Actor Folders
		else if (ULevel* OwningLevel = Cast<ULevel>(ParentPath.GetRootObjectPtr()))
		{
			if (FActorMode::IsActorLevelDisplayable(OwningLevel))
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(OwningLevel))
				{
					return *ParentItem;
				}
				else
				{
					return bCreate ? Mode->CreateItemFor<FLevelTreeItem>(OwningLevel, true) : nullptr;
				}
			}
		}
	}
	else if (const FComponentTreeItem* ComponentTreeItem = Item.CastTo<FComponentTreeItem>())
	{
		if (AActor* Owner = ComponentTreeItem->Component->GetOwner())
		{
			if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(Owner))
			{
				return *ParentItem;
			}
			else if (Owner->IsListedInSceneOutliner())
			{
				return bCreate ? CreateItemForActor(Owner, true) : nullptr;
			}
		}
		// do not default to world on Component items
		return nullptr;
	}
	else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
	{
		if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
		{
			if (UWorld* RepresentingWorldPtr = RepresentingWorld.Get())
			{
				const FFolder ActorDescFolder = FActorFolders::GetActorDescInstanceFolder(*RepresentingWorldPtr, ActorDescInstance);
				if (!ActorDescFolder.IsNone())
				{
					if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ActorDescFolder))
					{
						return *ParentItem;
					}
					else
					{
						return bCreate ? CreateItemForFolder(ActorDescFolder, RepresentingWorldPtr, true) : nullptr;
					}
				}
			}

			// Parent Actor (Actor attachement / parenting)
			const FGuid& ParentActorGuid = ActorDescInstance->GetSceneOutlinerParent();
			if (ParentActorGuid.IsValid())
			{
				if (UActorDescContainerInstance* ContainerInstance = ActorDescInstance->GetContainerInstance())
				{
					if (const FWorldPartitionActorDescInstance* ParentActorDesc = ContainerInstance->GetActorDescInstance(ParentActorGuid))
					{
						// If parent actor is loaded
						if (AActor* ParentActor = ParentActorDesc->GetActor())
						{
							// Find loaded parent actor node (from the object ptr)
							if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentActor))
							{
								return *ParentItem;
							}
							else
							{
								return bCreate ? CreateItemForActor(ParentActor, true) : nullptr;
							}
						}

						// Find unloaded parent actor node (from the guid)
						if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(FActorDescTreeItem::ComputeTreeItemID(ParentActorGuid, ContainerInstance)))
						{
							return *ParentItem;
						}
						else
						{
							return bCreate ? Mode->CreateItemFor<FActorDescTreeItem>(FActorDescTreeItem(ParentActorGuid, ContainerInstance)) : nullptr;
						}
					}
				}
			}
			else if (UActorDescContainerInstance* ContainerInstance = ActorDescInstance->GetContainerInstance())
			{
				const ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(RepresentingWorld.Get());
				UWorld* OuterWorld = ContainerInstance->GetTypedOuter<UWorld>();
				// If parent actor is loaded
				if (AActor* ParentActor = OuterWorld ? Cast<AActor>(LevelInstanceSubsystem->GetOwningLevelInstance(OuterWorld->PersistentLevel)) : nullptr)
				{
					if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentActor))
					{
						return *ParentItem;
					}
					else
					{
						return bCreate ? CreateItemForActor(ParentActor, true) : nullptr;
					}
				}
				// If parent actor is not loaded
				else if (UActorDescContainerInstance* ParentContainerInstance = Cast<UActorDescContainerInstance>(ContainerInstance->GetOuter()))
				{
					if (const FWorldPartitionActorDescInstance* ParentActorDescInstance = ParentContainerInstance->GetActorDescInstance(ContainerInstance->GetContainerActorGuid()))
					{
						if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(FActorDescTreeItem::ComputeTreeItemID(ParentActorDescInstance->GetGuid(), ParentContainerInstance)))
						{
							return *ParentItem;
						}
						else
						{
							return bCreate ? CreateItemForActorDescInstance(ParentActorDescInstance, true) : nullptr;
						}
					}
				}
			}
		}
	}

	// If we get here return world item
	if (UWorld* RepresentingWorldPtr = RepresentingWorld.Get())
	{
		if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(RepresentingWorldPtr))
		{
			return *ParentItem;
		}
		else
		{
			return bCreate ? Mode->CreateItemFor<FWorldTreeItem>(RepresentingWorldPtr, true) : nullptr;
		}
	}

	return nullptr;
}

void FActorHierarchy::CreateComponentItems(const AActor* Actor, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	check(Actor);
	// Add all this actors components if showing components and the owning actor was created
	if (bShowingComponents)
	{
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (Component != nullptr && !Component->GetClass()->HasAnyClassFlags(CLASS_Hidden))
			{
				if (FSceneOutlinerTreeItemPtr ComponentItem =
					Mode->CreateItemFor<FComponentTreeItem>(FComponentTreeItem(Component, bSearchComponentsByActorName)))
				{
					OutItems.Add(ComponentItem);
				}
			}
		}
	}
}

bool FActorHierarchy::IsShowingUnloadedActors() const
{
	return bShowingUnloadedActors && RepresentingWorld.IsValid() && !RepresentingWorld->IsPlayInEditor();
}

void FActorHierarchy::CreateFolderItems(UWorld* InWorld, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	// In game world, create folder through FindOrCreateParentItem
	if (InWorld->IsGameWorld())
	{
		return;
	}

	if (Mode->ShouldShowFolders() && bShowingEmptyFolders)
	{
		// Add any folders which might match the current search terms
		FActorFolders::Get().ForEachFolder(*InWorld, [this, &InWorld, &OutItems](const FFolder& Folder)
		{
			if (FSceneOutlinerTreeItemPtr FolderItem = CreateItemForFolder(Folder, InWorld))
			{
				OutItems.Add(FolderItem);
			}
			return true;
		});
	}
}

FSceneOutlinerTreeItemPtr FActorHierarchy::CreateItemForActor(AActor* InActor, bool bForce) const
{
	return Mode->CreateItemFor<FActorTreeItem>(InActor, bForce);
}

FSceneOutlinerTreeItemPtr FActorHierarchy::CreateItemForActorDescInstance(const FWorldPartitionActorDescInstance* InActorDescInstance, bool bForce) const
{
	return Mode->CreateItemFor<FActorDescTreeItem>(InActorDescInstance, bForce);
}

FSceneOutlinerTreeItemPtr FActorHierarchy::CreateItemForFolder(const FFolder& InFolder, const TWeakObjectPtr<UWorld>& InWorld, bool bForce) const
{
	return Mode->CreateItemFor<FActorFolderTreeItem>(FActorFolderTreeItem(InFolder, InWorld), bForce);
}

bool FActorHierarchy::CheckLevelInstanceEditing(UWorld* World, AActor* Actor) const
{
	const ULevelInstanceSubsystem* LevelInstanceSubsystem = World->GetSubsystem<ULevelInstanceSubsystem>();
	// If we are not showing LevelInstances, LevelInstance sub actor items should not be created unless they belong to a LevelInstance which is being edited
	if (LevelInstanceSubsystem)
	{
		if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
		{
			if (!bShowingLevelInstances && !ParentLevelInstance->IsEditing() && !ParentLevelInstance->IsEditingPropertyOverrides())
			{
				return false;
			}
		}
	}

	return true;
}

void FActorHierarchy::InsertActorItemAndCreateComponents(AActor* InActor, FSceneOutlinerTreeItemPtr ActorItem, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	const int32 InsertLocation = OutItems.Num();
	
	// Create all component items
	CreateComponentItems(InActor, OutItems);

	// If we are only showing actors with valid components, don't add the actor if no components were added
	if (bShowingOnlyActorWithValidComponents)
	{
		// If the number of items remained the same after component insertion no components were inserted
		if (OutItems.Num() == InsertLocation) 
		{
			return;
		}
	}

	if (ActorItem)
	{
		// Add the actor before the components
		OutItems.Insert(ActorItem, InsertLocation);
	}
}

void FActorHierarchy::CreateUnloadedItems(UWorld* World, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	TFunction<void(const FWorldPartitionActorDescInstance*)> AddChildContainer = [this, &OutItems, &AddChildContainer](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		FWorldPartitionActorDesc::FContainerInstance SubContainerInstance;
		if (ActorDescInstance->GetChildContainerInstance(SubContainerInstance))
		{
			for (FActorDescInstanceList::TIterator<> It(SubContainerInstance.ContainerInstance); It; ++It)
			{
				const FWorldPartitionActorDescInstance* SubActorDescInstance = *It;

				if (SubActorDescInstance && FActorDescTreeItem::ShouldDisplayInOutliner(SubActorDescInstance))
				{
					if (FSceneOutlinerTreeItemPtr SubActorDescItem = Mode->CreateItemFor<FActorDescTreeItem>(SubActorDescInstance))
					{
						OutItems.Add(SubActorDescItem);
					}

					AddChildContainer(SubActorDescInstance);
				}
			}
		}
	};

	if (IsShowingUnloadedActors())
	{
		if (UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(World))
		{
			const ULevelInstanceSubsystem* LevelInstanceSubsystem = World->GetSubsystem<ULevelInstanceSubsystem>();

			WorldPartitionSubsystem->ForEachWorldPartition([this, LevelInstanceSubsystem, &OutItems, &AddChildContainer](UWorldPartition* WorldPartition)
			{
				UWorld* OuterWorld = WorldPartition->GetTypedOuter<UWorld>();
				ULevel* OuterLevel = OuterWorld ? OuterWorld->PersistentLevel.Get() : nullptr;

				// Skip unloaded actors if they are part of a non editing level instance and the outliner hides the content of level instances
				if (!bShowingLevelInstances)
				{
					ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem ? LevelInstanceSubsystem->GetOwningLevelInstance(OuterLevel) : nullptr;
					if (LevelInstance && !LevelInstance->IsEditing())
					{
						return true;
					}
				}

				const TMap<FGuid, AActor*> LoadedActors = FWorldPartitionHelpers::GetLoadedActorsForLevel(OuterLevel);
				const TMap<FGuid, AActor*> RegisteredActors = FWorldPartitionHelpers::GetRegisteredActorsForLevel(OuterLevel);

				FWorldPartitionHelpers::ForEachActorDescInstance(WorldPartition, [this, &LoadedActors, &RegisteredActors, &OutItems, &AddChildContainer, LevelInstanceSubsystem](const FWorldPartitionActorDescInstance* ActorDescInstance)
				{
					if (ActorDescInstance && FActorDescTreeItem::ShouldDisplayInOutliner(ActorDescInstance))
					{
						if (AActor* const* LoadedActor = LoadedActors.Find(ActorDescInstance->GetGuid()))
						{
							if (!IsValid(*LoadedActor) || RegisteredActors.Contains(ActorDescInstance->GetGuid()))
							{
								return true;
							}
						}

						UActorDescContainerInstance* ContainerInstance = ActorDescInstance->GetContainerInstance();

						if (const FSceneOutlinerTreeItemPtr ActorDescItem = Mode->CreateItemFor<FActorDescTreeItem>(FActorDescTreeItem(ActorDescInstance->GetGuid(), ContainerInstance)))
						{
							OutItems.Add(ActorDescItem);
						}

						if (bShowingLevelInstances)
						{
							// If parent actor is not loaded, recurse
							UWorld* OuterWorld = ContainerInstance->GetTypedOuter<UWorld>();
							if (AActor* ParentActor = OuterWorld ? Cast<AActor>(LevelInstanceSubsystem->GetOwningLevelInstance(OuterWorld->PersistentLevel)) : nullptr; !ParentActor)
							{
								AddChildContainer(ActorDescInstance);
							}
						}
					}
					return true;
				});
				return true;
			});
		}
	}
}

void FActorHierarchy::CreateWorldChildren(UWorld* World, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FActorHierarchy::CreateWorldChildren);

	check(World);

	CreateFolderItems(World, OutItems);
	
	// Create all actor items
	for (FActorIterator ActorIt(World); ActorIt; ++ActorIt)
	{
		AActor* Actor = *ActorIt;

		if (!CheckLevelInstanceEditing(World, Actor))
		{
			continue;
		}

		FSceneOutlinerTreeItemPtr ActorItem = CreateItemForActor(Actor);
		
		InsertActorItemAndCreateComponents(Actor, ActorItem, OutItems);
		
	}

	CreateUnloadedItems(World, OutItems);
}

void FActorHierarchy::CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FActorHierarchy::CreateItems);

	if (RepresentingWorld.IsValid())
	{
		UWorld* RepresentingWorldPtr = RepresentingWorld.Get();
		check(RepresentingWorldPtr);
		if (FSceneOutlinerTreeItemPtr WorldItem = Mode->CreateItemFor<FWorldTreeItem>(RepresentingWorldPtr))
		{
			OutItems.Add(WorldItem);
		}
		// Create world children regardless of if a world item was created
		CreateWorldChildren(RepresentingWorldPtr, OutItems);
	}
}

void FActorHierarchy::CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const
{
	auto CreateChildrenFolders = [this](UWorld* InWorld, const FFolder& InParentFolder, const FFolder::FRootObject& InFolderRootObject, TArray<FSceneOutlinerTreeItemPtr>& OutChildren)
	{
		// In game world, create folder through FindOrCreateParentItem
		if (InWorld->IsGameWorld())
		{
			return;
		}

		FActorFolders::Get().ForEachFolderWithRootObject(*InWorld, InFolderRootObject, [this, InWorld, &InParentFolder, &OutChildren](const FFolder& Folder)
		{
			if (Folder.IsChildOf(InParentFolder))
			{
				if (FSceneOutlinerTreeItemPtr NewFolderItem = CreateItemForFolder(Folder, InWorld))
				{
					OutChildren.Add(NewFolderItem);
				}
			}
			return true;
		});
	};

	UWorld* World = RepresentingWorld.Get();
	if (FWorldTreeItem* WorldItem = Item->CastTo<FWorldTreeItem>())
	{
		check(WorldItem->World == RepresentingWorld);
		CreateWorldChildren(WorldItem->World.Get(), OutChildren);
	}
	else if (const FActorTreeItem* ParentActorItem = Item->CastTo<FActorTreeItem>())
	{
		AActor* ParentActor = ParentActorItem->Actor.Get();
		check(ParentActor->GetWorld() == RepresentingWorld);

		CreateComponentItems(ParentActor, OutChildren);

		TArray<AActor*> ChildActors;

		if (const ILevelInstanceInterface* LevelInstanceParent = Cast<ILevelInstanceInterface>(ParentActor))
		{
			const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
			check(LevelInstanceSubsystem);

			LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstanceParent, [this, LevelInstanceParent, LevelInstanceSubsystem, &ChildActors](AActor* SubActor)
			{
				const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(SubActor);
				const bool bIsAnEditingLevelInstance = LevelInstance ? LevelInstanceSubsystem->IsEditingLevelInstance(LevelInstance) : false;
				if (bShowingLevelInstances || (LevelInstanceSubsystem->IsEditingLevelInstance(LevelInstanceParent) || bIsAnEditingLevelInstance))
				{
					ChildActors.Add(SubActor);
				}
				return true;
			});

			check(World == CastChecked<AActor>(LevelInstanceParent)->GetWorld());
			FFolder ParentFolder = ParentActor->GetFolder();
			CreateChildrenFolders(World, ParentFolder, ParentActor, OutChildren);
		}
		else
		{
			// NOTE: GetAttachedActors can sometimes return the actor itself or its parent depending on how the component ownership is setup
			// but we don't have to check for that here because the Outliner will simply discard duplicates
			ParentActor->GetAttachedActors(ChildActors, true, true);
		}

		for (auto ChildActor : ChildActors)
		{
			if (FSceneOutlinerTreeItemPtr ChildActorItem = CreateItemForActor(ChildActor))
			{
				OutChildren.Add(ChildActorItem);

				CreateComponentItems(ChildActor, OutChildren);
			}
		}
	}
	else if (FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>())
	{
		check(Mode->ShouldShowFolders());
		
		check(World == FolderItem->World.Get());
		FFolder ParentFolder = FolderItem->GetFolder();
		check(!ParentFolder.IsNone());
		CreateChildrenFolders(World, ParentFolder, ParentFolder.GetRootObject(), OutChildren);
	}
}

void FActorHierarchy::FullRefreshEvent()
{
	FSceneOutlinerHierarchyChangedData EventData;
	EventData.Type = FSceneOutlinerHierarchyChangedData::FullRefresh;

	HierarchyChangedEvent.Broadcast(EventData);
}

void FActorHierarchy::OnWorldPartitionCreated(UWorld* InWorld)
{
	if (RepresentingWorld.Get() == InWorld)
	{
		FullRefreshEvent();
	}
}

void FActorHierarchy::OnLevelActorAdded(AActor* InActor)
{
	if (InActor != nullptr && RepresentingWorld.Get() == InActor->GetWorld())
	{
		// CreateItemForActor already checks this by calling FActorMode::IsActorDisplayable, but that is after the actor tree item has
		// already been created so this is a small optimization to avoid creating it and then calling Actor->GetActorLabel() which can be expensive
		if (InActor->IsListedInSceneOutliner())
		{
			FSceneOutlinerHierarchyChangedData EventData;
			EventData.Type = FSceneOutlinerHierarchyChangedData::Added;
			EventData.Items.Add(CreateItemForActor(InActor));
			HierarchyChangedEvent.Broadcast(EventData);
		}
	}
}

void FActorHierarchy::OnLevelActorDeleted(AActor* InActor)
{
	if (InActor != nullptr && RepresentingWorld.Get() == InActor->GetWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
		EventData.ItemIDs.Add(InActor);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FActorHierarchy::OnLevelActorAttached(AActor* InActor, const AActor* InParent)
{
	if (InActor != nullptr && RepresentingWorld.Get() == InActor->GetWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Moved;
		EventData.ItemIDs.Add(InActor);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FActorHierarchy::OnLevelActorDetached(AActor* InActor, const AActor* InParent)
{
	if (InActor != nullptr && RepresentingWorld.Get() == InActor->GetWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Moved;
		EventData.ItemIDs.Add(InActor);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FActorHierarchy::RemoveActorDesc(AActor& InActor)
{
	// Loaded actors can be in sub-world partitions so we use the outer world
	if (UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(&InActor))
	{
		const FGuid& ActorGuid = InActor.GetActorGuid();
		if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid))
		{
			FSceneOutlinerHierarchyChangedData EventData;
			EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
			EventData.ItemIDs.Add(FActorDescTreeItem::ComputeTreeItemID(ActorGuid, ActorDescInstance->GetContainerInstance()));
			HierarchyChangedEvent.Broadcast(EventData);
		}
	}
}

void FActorHierarchy::AddActorDesc(AActor& InActor)
{
	if (IsShowingUnloadedActors())
	{
		if (UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(&InActor))
		{
			const FGuid& ActorGuid = InActor.GetActorGuid();
			if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid); FActorDescTreeItem::ShouldDisplayInOutliner(ActorDescInstance))
			{
				FSceneOutlinerHierarchyChangedData EventData;
				EventData.Type = FSceneOutlinerHierarchyChangedData::Added;
				EventData.Items.Add(Mode->CreateItemFor<FActorDescTreeItem>(FActorDescTreeItem(ActorDescInstance->GetGuid(), ActorDescInstance->GetContainerInstance())));
				HierarchyChangedEvent.Broadcast(EventData);
			}
		}
	}
}

void FActorHierarchy::OnLoadedActorAdded(AActor& InActor)
{
	OnLevelActorAdded(&InActor);

	RemoveActorDesc(InActor);
}

void FActorHierarchy::OnLoadedActorRemoved(AActor& InActor)
{
	OnLevelActorDeleted(&InActor);

	AddActorDesc(InActor);
}

void FActorHierarchy::OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (IsShowingUnloadedActors() && InActorDescInstance && !InActorDescInstance->IsLoaded(true) && FActorDescTreeItem::ShouldDisplayInOutliner(InActorDescInstance))
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Added;
		EventData.Items.Add(Mode->CreateItemFor<FActorDescTreeItem>(FActorDescTreeItem(InActorDescInstance->GetGuid(), InActorDescInstance->GetContainerInstance())));
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FActorHierarchy::OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (IsShowingUnloadedActors() && InActorDescInstance)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
		EventData.ItemIDs.Add(FActorDescTreeItem::ComputeTreeItemID(InActorDescInstance->GetGuid(), InActorDescInstance->GetContainerInstance()));
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FActorHierarchy::OnWorldPartitionInitialized(UWorldPartition* InWorldPartition)
{
	FullRefreshEvent();
}

void FActorHierarchy::OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
{
	FullRefreshEvent();
}

void FActorHierarchy::OnComponentsUpdated()
{
	FullRefreshEvent();
}

void FActorHierarchy::OnLevelActorListChanged()
{
	FullRefreshEvent();
}

void FActorHierarchy::OnActorFoldersUpdatedEvent(ULevel* InLevel)
{
	FullRefreshEvent();
}

void FActorHierarchy::OnActorFolderAdded(UActorFolder* InActorFolder)
{
	check(InActorFolder);
	ULevel* Level = InActorFolder->GetOuterULevel();
	if (Level && Mode->ShouldShowFolders() && (RepresentingWorld.Get() == Level->GetWorld()))
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Added;
		EventData.Items.Add(CreateItemForFolder(InActorFolder->GetFolder(), RepresentingWorld));
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FActorHierarchy::OnLevelAdded(ULevel* InLevel, UWorld* InWorld)
{
	if (InLevel != nullptr && RepresentingWorld.Get() == InWorld)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Added;

		EventData.Items.Reserve(InLevel->Actors.Num());
		for (AActor* Actor : InLevel->Actors)
		{
			if (Actor != nullptr)
			{
				EventData.Items.Add(CreateItemForActor(Actor));
			}
		}
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FActorHierarchy::RemoveLevelActorFolders(ULevel* InLevel, UWorld* InWorld)
{
	// If either this level or the owning world are using actor folders, remove level's actor folders
	if (InLevel->IsUsingActorFolders() || InWorld->PersistentLevel->IsUsingActorFolders())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
		EventData.ItemIDs.Add(InLevel);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FActorHierarchy::OnLevelRemoved(ULevel* InLevel, UWorld* InWorld)
{
	if (InLevel != nullptr && RepresentingWorld.Get() == InWorld)
	{
		{
			FSceneOutlinerHierarchyChangedData EventData;
			EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
			EventData.ItemIDs.Reserve(InLevel->Actors.Num());
			for (AActor* Actor : InLevel->Actors)
			{
				if (Actor != nullptr)
				{
					EventData.ItemIDs.Add(Actor);
				}
			}
			HierarchyChangedEvent.Broadcast(EventData);
		}

		RemoveLevelActorFolders(InLevel, InWorld);
	}
}

/** Called when a folder is to be created */
void FActorHierarchy::OnBroadcastFolderCreate(UWorld& InWorld, const FFolder& InNewFolder)
{
	if (Mode->ShouldShowFolders() && RepresentingWorld.Get() == &InWorld)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Added;
		EventData.Items.Add(CreateItemForFolder(InNewFolder, &InWorld));
		EventData.ItemActions = SceneOutliner::ENewItemAction::Select | SceneOutliner::ENewItemAction::Rename;
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

/** Called when a folder is to be moved */
void FActorHierarchy::OnBroadcastFolderMove(UWorld& InWorld, const FFolder& InOldFolder, const FFolder& InNewFolder)
{
	if (Mode->ShouldShowFolders() && RepresentingWorld.Get() == &InWorld)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::FolderMoved;
		EventData.ItemIDs.Add(InOldFolder);
		EventData.NewPaths.Add(InNewFolder);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

/** Called when a folder is to be deleted */
void FActorHierarchy::OnBroadcastFolderDelete(UWorld& InWorld, const FFolder& InFolder)
{
	if (Mode->ShouldShowFolders() && RepresentingWorld.Get() == &InWorld)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
		EventData.ItemIDs.Add(InFolder);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FActorHierarchy::OnLevelActorFolderChanged(const AActor* InActor, FName OldPath)
{
	if (Mode->ShouldShowFolders() && RepresentingWorld.Get() == InActor->GetWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Moved;
		EventData.ItemIDs.Add(FSceneOutlinerTreeItemID(InActor));
		HierarchyChangedEvent.Broadcast(EventData);
	}
}
