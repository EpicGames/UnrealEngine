// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorMode.h"
#include "ActorHierarchy.h"
#include "Engine/Selection.h"
#include "Editor/UnrealEdEngine.h"
#include "SceneOutlinerDelegates.h"
#include "ActorEditorUtils.h"
#include "LevelUtils.h"
#include "GameFramework/WorldSettings.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ActorTreeItem.h"
#include "LevelTreeItem.h"
#include "FolderTreeItem.h"
#include "ComponentTreeItem.h"
#include "ActorDescTreeItem.h"
#include "EditorActorFolders.h"
#include "EditorFolderUtils.h"
#include "EditorLevelUtils.h"
#include "EditorModeManager.h"
#include "Engine/GameViewportClient.h"
#include "WorldTreeItem.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "LevelEditor.h"
#include "SceneOutlinerHelpers.h"
#include "Logging/MessageLog.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "SSocketChooser.h"
#include "UnrealClient.h"

DEFINE_LOG_CATEGORY_STATIC(LogActorMode, Log, All);

#define LOCTEXT_NAMESPACE "SceneOutliner_ActorMode"

using FActorFilter = TSceneOutlinerPredicateFilter<FActorTreeItem>;
using FFolderFilter = TSceneOutlinerPredicateFilter<FFolderTreeItem>;
using FComponentFilter = TSceneOutlinerPredicateFilter<FComponentTreeItem>;

namespace SceneOutliner
{
	bool FWeakActorSelector::operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, TWeakObjectPtr<AActor>& DataOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FActorTreeItem* ActorItem = ItemPtr->CastTo<FActorTreeItem>())
			{
				if (ActorItem->IsValid())
				{
					DataOut = ActorItem->Actor;
					return true;
				}
			}
		}
		return false;
	}

	bool FActorSelector::operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, AActor*& ActorPtrOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FActorTreeItem* ActorItem = ItemPtr->CastTo<FActorTreeItem>())
			{
				if (ActorItem->IsValid())
				{
					AActor* Actor = ActorItem->Actor.Get();
					if (Actor)
					{
						ActorPtrOut = Actor;
						return true;
					}
				}
			}
			// If a component is selected, we meant for the owning actor to be selected
			else if (FComponentTreeItem* ComponentItem = ItemPtr->CastTo<FComponentTreeItem>())
			{
				if (ComponentItem->IsValid())
				{
					AActor* Actor = ComponentItem->Component->GetOwner();
					if (Actor)
					{
						ActorPtrOut = Actor;
						return true;
					}
				}
			}
		}

		return false;
	}

	bool FActorHandleSelector::operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, FWorldPartitionHandle& ActorHandleOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FActorDescTreeItem* ActorDescItem = ItemPtr->CastTo<FActorDescTreeItem>())
			{
				ActorHandleOut = ActorDescItem->ActorDescHandle;
				return true;
			}
		}

		return false;
	}
}

FActorMode::FActorMode(const FActorModeParams& Params)
	: ISceneOutlinerMode(Params.SceneOutliner)
	, SpecifiedWorldToDisplay(Params.SpecifiedWorldToDisplay)
	, bHideComponents(Params.bHideComponents)
	, bHideActorWithNoComponent(Params.bHideActorWithNoComponent)
	, bHideLevelInstanceHierarchy(Params.bHideLevelInstanceHierarchy)
	, bHideUnloadedActors(Params.bHideUnloadedActors)
	, bHideEmptyFolders(Params.bHideEmptyFolders)
	, bCanInteractWithSelectableActorsOnly(Params.bCanInteractWithSelectableActorsOnly)
	, bShouldUpdateContentWhileInPIEFocused(Params.bShouldUpdateContentWhileInPIEFocused)
	, bSearchComponentsByActorName(Params.bSearchComponentsByActorName)
{
	SceneOutliner->AddFilter(MakeShared<FActorFilter>(FActorTreeItem::FFilterPredicate::CreateLambda([this](const AActor* Actor)
	{
		return IsActorDisplayable(Actor);
	}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	// Don't show components if the owner actor is not displayable
	SceneOutliner->AddFilter(MakeShared<FComponentFilter>(FComponentTreeItem::FFilterPredicate::CreateLambda([this](const UActorComponent* ActorComponent)
	{
		return ActorComponent && IsActorDisplayable(ActorComponent->GetOwner());
	}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));


	auto FolderPassesFilter = [this](const FFolder& InFolder, bool bInCheckHideLevelInstanceFlag)
	{
		if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(InFolder.GetRootObjectPtr()))
		{
			if (LevelInstance->IsEditing() || LevelInstance->IsEditingPropertyOverrides())
			{
				return true;
			}
			if (bInCheckHideLevelInstanceFlag)
			{
				return !bHideLevelInstanceHierarchy;
			}
		}
		if (ULevel* Level = Cast<ULevel>(InFolder.GetRootObjectPtr()))
		{
			return true;
		}
		return false;
	};

	SceneOutliner->AddFilter(MakeShared<FFolderFilter>(FFolderTreeItem::FFilterPredicate::CreateLambda([FolderPassesFilter](const FFolder& InFolder)
	{
		return FolderPassesFilter(InFolder, /*bCheckHideLevelInstanceFlag*/true);
	}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	SceneOutliner->AddInteractiveFilter(MakeShared<FFolderFilter>(FFolderTreeItem::FFilterPredicate::CreateLambda([FolderPassesFilter](const FFolder& InFolder)
	{
		return FolderPassesFilter(InFolder, /*bCheckHideLevelInstanceFlag*/false);
	}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

FActorMode::~FActorMode()
{
}

TUniquePtr<ISceneOutlinerHierarchy> FActorMode::CreateHierarchy()
{
	TUniquePtr<FActorHierarchy> ActorHierarchy = FActorHierarchy::Create(this, RepresentingWorld);
	ActorHierarchy->SetShowingComponents(!bHideComponents);
	ActorHierarchy->SetShowingOnlyActorWithValidComponents(!bHideComponents && bHideActorWithNoComponent);
	ActorHierarchy->SetShowingLevelInstances(!bHideLevelInstanceHierarchy);
	ActorHierarchy->SetShowingUnloadedActors(!bHideUnloadedActors);
	ActorHierarchy->SetShowingEmptyFolders(!bHideEmptyFolders);
	ActorHierarchy->SetSearchComponentsByActorName(bSearchComponentsByActorName);

	return ActorHierarchy;
}

void FActorMode::Rebuild()
{
	ChooseRepresentingWorld();

	Hierarchy = CreateHierarchy();
}

void FActorMode::ChooseRepresentingWorld()
{
	SceneOutliner::FSceneOutlinerHelpers::ChooseRepresentingWorld(RepresentingWorld, SpecifiedWorldToDisplay, UserChosenWorld);
}

void FActorMode::BuildWorldPickerMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Worlds", LOCTEXT("WorldsHeading", "Worlds"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoWorld", "Auto"),
			LOCTEXT("AutoWorldToolTip", "Automatically pick the world to display based on context."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FActorMode::OnSelectWorld, TWeakObjectPtr<UWorld>()),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FActorMode::IsWorldChecked, TWeakObjectPtr<UWorld>())
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && (World->WorldType == EWorldType::PIE || Context.WorldType == EWorldType::Editor))
			{
				MenuBuilder.AddMenuEntry(
					SceneOutliner::GetWorldDescription(World),
					LOCTEXT("ChooseWorldToolTip", "Display actors for this world."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateRaw(this, &FActorMode::OnSelectWorld, MakeWeakObjectPtr(World)),
						FCanExecuteAction(),
						FIsActionChecked::CreateRaw(this, &FActorMode::IsWorldChecked, MakeWeakObjectPtr(World))
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
	MenuBuilder.EndSection();
}

void FActorMode::OnSelectWorld(TWeakObjectPtr<UWorld> World)
{
	UserChosenWorld = World;
	SceneOutliner->FullRefresh();
}

bool FActorMode::IsWorldChecked(TWeakObjectPtr<UWorld> World) const
{
	return (UserChosenWorld == World) || (World.IsExplicitlyNull() && !UserChosenWorld.IsValid());
}

void FActorMode::SynchronizeActorSelection()
{
	USelection* SelectedActors = GEditor->GetSelectedActors();

	// Deselect actors in the tree that are no longer selected in the world
	const FSceneOutlinerItemSelection Selection(SceneOutliner->GetSelection());
	auto DeselectActors = [this](FActorTreeItem& Item)
	{
		if (!Item.Actor.IsValid() || !Item.Actor.Get()->IsSelected())
		{
			SceneOutliner->SetItemSelection(Item.AsShared(), false);
		}
	};
	Selection.ForEachItem<FActorTreeItem>(DeselectActors);

	// Show actor selection but only if sub objects are not selected
	if (!Selection.Has<FComponentTreeItem>())
	{
		// See if the tree view selector is pointing at a selected item
		bool bSelectorInSelectionSet = false;

		TArray<FSceneOutlinerTreeItemPtr> ActorItems;
		for (FSelectionIterator SelectionIt(*SelectedActors); SelectionIt; ++SelectionIt)
		{
			AActor* Actor = CastChecked< AActor >(*SelectionIt);
			if (FSceneOutlinerTreeItemPtr ActorItem = SceneOutliner->GetTreeItem(Actor))
			{
				if (!bSelectorInSelectionSet && SceneOutliner->HasSelectorFocus(ActorItem))
				{
					bSelectorInSelectionSet = true;
				}

				ActorItems.Add(ActorItem);
			}
		}

		// If NOT bSelectorInSelectionSet then we want to just move the selector to the first selected item.
		ESelectInfo::Type SelectInfo = bSelectorInSelectionSet ? ESelectInfo::Direct : ESelectInfo::OnMouseClick;
		SceneOutliner->AddToSelection(ActorItems, SelectInfo);
	}
	FSceneOutlinerDelegates::Get().SelectionChanged.Broadcast();
}

bool FActorMode::IsActorDisplayable(const AActor* Actor) const
{
	return FActorMode::IsActorDisplayable(SceneOutliner, Actor, !bHideLevelInstanceHierarchy);
}

FFolder::FRootObject FActorMode::GetRootObject() const
{
	return FFolder::GetWorldRootFolder(RepresentingWorld.Get()).GetRootObject();
}

FFolder::FRootObject FActorMode::GetPasteTargetRootObject() const
{
	if (UWorld* World = RepresentingWorld.Get())
	{
		return FFolder::GetOptionalFolderRootObject(World->GetCurrentLevel()).Get(FFolder::GetWorldRootFolder(World).GetRootObject());
	}
	return FFolder::GetInvalidRootObject();
}

bool FActorMode::IsActorDisplayable(const SSceneOutliner* SceneOutliner, const AActor* Actor, bool bShowLevelInstanceContent)
{
	const bool bIsActorDisplayable = 	FActorEditorUtils::IsActorDisplayable(Actor) &&
										!SceneOutliner->GetSharedData().bOnlyShowFolders;		// Don't show actors if we're only showing folders
	
	if(bIsActorDisplayable && Actor->HasAnyFlags(RF_Transient))
	{
		// Level Instance transient actors are shown based on passed in bShowLevelInstanceContent flag
		if (Actor->IsInLevelInstance())
		{
			return bShowLevelInstanceContent;
		}
		else
		{
			// Don't show transient actors in non-play worlds, except if bShowTransient is true
			return SceneOutliner->GetSharedData().bShowTransient || (Actor->GetWorld() && Actor->GetWorld()->IsPlayInEditor());
		}
	}

	return	bIsActorDisplayable;
}

bool FActorMode::CanInteract(const ISceneOutlinerTreeItem& Item) const
{
	if (bCanInteractWithSelectableActorsOnly)
	{
		AActor* FoundActor = nullptr;
		if (const FActorTreeItem* ActorTreeItem = Item.CastTo<FActorTreeItem>())
		{
			FoundActor = ActorTreeItem->Actor.Get();
		}
		else if (const FComponentTreeItem* ComponentTreeItem = Item.CastTo<FComponentTreeItem>())
		{
			if (UActorComponent* Component = ComponentTreeItem->Component.Get())
			{
				FoundActor = Component->GetOwner();
			}
		}

		if (FoundActor)
		{
			const bool bInSelected = true;
			const bool bSelectEvenIfHidden = true;		// @todo outliner: Is this actually OK?
			if (!GEditor->CanSelectActor(FoundActor, bInSelected, bSelectEvenIfHidden))
			{
				return false;
			}
		}
	}
	
	return true;
}

bool FActorMode::CanPopulate() const
{
	if (!bShouldUpdateContentWhileInPIEFocused)
	{
		if (UGameViewportClient* GameViewport = RepresentingWorld->GetGameViewport())
		{
			return !GameViewport->Viewport || !GameViewport->Viewport->HasFocus();
		}
	}
	return true;
}

bool FActorMode::IsActorLevelDisplayable(ULevel* InLevel)
{
	// Don't show level tree item for the persistent level
	return (InLevel && !InLevel->IsPersistentLevel());
}

void FActorMode::OnFilterTextChanged(const FText& InFilterText)
{
	// Scroll last item (if it passes the filter) into view - this means if we are multi-selecting, we show newest selection that passes the filter
	if (const AActor* LastSelectedActor = GEditor->GetSelectedActors()->GetBottom<AActor>())
	{
		// This part is different than that of OnLevelSelectionChanged(nullptr) because IsItemVisible(TreeItem) & ScrollItemIntoView(TreeItem) are applied to
		// the current visual state, not to the one after applying the filter. Thus, the scroll would go to the place where the object was located
		// before applying the FilterText

		// If the object is already in the list, but it does not passes the filter, then we do not want to re-add it, because it will be removed by the filter
		const FSceneOutlinerTreeItemPtr TreeItem = SceneOutliner->GetTreeItem(LastSelectedActor);
		if (!TreeItem.IsValid() || !SceneOutliner->PassesTextFilter(TreeItem))
		{
			return;
		}

		// If the object is not in the list, and it does not passes the filter, then we should not re-add it, because it would be removed by the filter again. Unfortunately,
		// there is no code to check if a future element (i.e., one that is currently not in the TreeItemMap list) will pass the filter. Therefore, we kind of overkill it
		// by re-adding that element (even though it will be removed). However, AddItemToTree(FSceneOutlinerTreeItemRef Item) and similar functions already check the element before
		// adding it. So this solution is fine.
		// This solution might affect the performance of the World Outliner when a key is pressed, but it will still work properly when the remove/del keys are pressed. Not
		// updating the filter when !TreeItem.IsValid() would result in the focus not being updated when the remove/del keys are pressed.

		// In any other case (i.e., if the object passes the current filter), re-add it
		SceneOutliner->ScrollItemIntoView(TreeItem);

		SetAsMostRecentOutliner();
	}
}

void FActorMode::SetAsMostRecentOutliner() const
{
	TWeakPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();

	if(TSharedPtr<ILevelEditor> LevelEditorPin = LevelEditor.Pin())
	{
		LevelEditorPin->SetMostRecentlyUsedSceneOutliner(SceneOutliner->GetOutlinerIdentifier());
	}
}

int32 FActorMode::GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const
{
	if (Item.IsA<FWorldTreeItem>())
	{
		return EItemSortOrder::World;
	}
	else if (Item.IsA<FLevelTreeItem>())
	{
		return EItemSortOrder::Level;
	}
	else if (Item.IsA<FFolderTreeItem>())
	{
		return EItemSortOrder::Folder;
	}
	else if (Item.IsA<FActorTreeItem>() || Item.IsA<FComponentTreeItem>() || Item.IsA<FActorDescTreeItem>())
	{
		return EItemSortOrder::Actor;
	}

	// Warning: using actor mode with an unsupported item type!
	check(false);
	return -1;
}

FSceneOutlinerDragValidationInfo FActorMode::ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const
{
	if (Payload.Has<FFolderTreeItem>())
	{
		FFolder::FRootObject TargetRootObject = DropTarget.GetRootObject();
		FFolder::FRootObject CommonPayloadFoldersRootObject;
		TArray<FName> PayloadFolders;
		const bool bHasCommonRootObject = GetFolderNamesFromPayload(Payload, PayloadFolders, CommonPayloadFoldersRootObject);
		if (!bHasCommonRootObject)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("CantMoveFoldersWithMultipleRoots", "Cannot move folders with multiple roots"));
		}
		else if (CommonPayloadFoldersRootObject != TargetRootObject)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("CantChangeFoldersRoot", "Cannot change folders root"));
		}
	}

	if (const FActorTreeItem* ActorItem = DropTarget.CastTo<FActorTreeItem>())
	{
		const AActor* ActorTarget = ActorItem->Actor.Get();
		if (!ActorTarget)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
		}

		const ILevelInstanceInterface* LevelInstanceTarget = Cast<ILevelInstanceInterface>(ActorTarget);
		const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
		if (LevelInstanceTarget)
		{
			check(LevelInstanceSubsystem);
			if (!LevelInstanceTarget->IsEditing())
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_AttachToClosedLevelInstance", "Cannot attach to LevelInstance which is not being edited"));
			}
		}
		else
		{
			if (Payload.Has<FFolderTreeItem>())
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("FoldersOnActorError", "Cannot attach folders to actors"));
			}

			if (!Payload.Has<FActorTreeItem>())
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
			}
		}

		FText AttachErrorMsg;
		bool bCanAttach = true;
		bool bDraggedOntoAttachmentParent = true;
		const auto& DragActors = Payload.GetData<TWeakObjectPtr<AActor>>(SceneOutliner::FWeakActorSelector());
		for (const auto& DragActorPtr : DragActors)
		{
			AActor* DragActor = DragActorPtr.Get();
			if (DragActor)
			{
				if (bCanAttach)
				{
					if (LevelInstanceSubsystem)
					{
						// Either all actors must be in a LevelInstance or none of them
						if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(DragActor))
						{
							if (!ParentLevelInstance->IsEditing())
							{
								return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_RemoveEditingLevelInstance", "Cannot detach from a LevelInstance which is not being edited"));
							}
						}

						if (!LevelInstanceSubsystem->CanMoveActorToLevel(DragActor, &AttachErrorMsg))
						{
							bCanAttach = bDraggedOntoAttachmentParent = false;
							break;
						}
					}

					if (DragActor->IsChildActor())
					{
						AttachErrorMsg = FText::Format(LOCTEXT("Error_AttachChildActor", "Cannot move {0} as it is a child actor."), FText::FromString(DragActor->GetActorLabel()));
						bCanAttach = bDraggedOntoAttachmentParent = false;
						break;
					}
					if (!LevelInstanceTarget && !GEditor->CanParentActors(ActorTarget, DragActor, &AttachErrorMsg))
					{
						bCanAttach = false;
					}
				}

				if (DragActor->GetSceneOutlinerParent() != ActorTarget)
				{
					bDraggedOntoAttachmentParent = false;
				}
			}
		}

		const FText ActorLabel = FText::FromString(ActorTarget->GetActorLabel());
		if (bDraggedOntoAttachmentParent)
		{
			for (const auto& DragActorPtr : DragActors)
			{
				AActor* DragActor = DragActorPtr.Get();
				if (!DragActor->EditorCanDetachFrom(DragActor->GetSceneOutlinerParent(), AttachErrorMsg))
				{
					// Cannot detach from parent into root
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, AttachErrorMsg); 
				}
			}

			if (DragActors.Num() == 1)
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleDetach, ActorLabel);
			}
			else
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleMultipleDetach, ActorLabel);
			}
		}
		else if (bCanAttach)
		{
			if (DragActors.Num() == 1)
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleAttach, ActorLabel);
			}
			else
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleMultipleAttach, ActorLabel);
			}
		}
		else
		{
			if (DragActors.Num() == 1)
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, AttachErrorMsg);
			}
			else
			{
				const FText ReasonText = FText::Format(LOCTEXT("DropOntoText", "{0}. {1}"), ActorLabel, AttachErrorMsg);
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleMultipleAttach, ReasonText);
			}
		}
	}
	else if (DropTarget.IsA<FFolderTreeItem>() || DropTarget.IsA<FWorldTreeItem>() || DropTarget.IsA<FLevelTreeItem>())
	{
		const FFolderTreeItem* FolderItem = DropTarget.CastTo<FFolderTreeItem>();
		const FWorldTreeItem* WorldItem = DropTarget.CastTo<FWorldTreeItem>();
		const FLevelTreeItem* LevelItem = DropTarget.CastTo<FLevelTreeItem>();
		// WorldTreeItem and LevelTreeItem are treated as root folders (path = none), with the difference that LevelTreeItem has a RootObject.
		const FFolder DestinationPath = FolderItem ? FolderItem->GetFolder() : (LevelItem ? FFolder(FFolder::GetOptionalFolderRootObject(LevelItem->Level.Get()).Get(FFolder::GetInvalidRootObject())) : GetWorldDefaultRootFolder());
		const FFolder::FRootObject& DestinationRootObject = DestinationPath.GetRootObject();
		ILevelInstanceInterface* LevelInstanceTarget = Cast<ILevelInstanceInterface>(DestinationPath.GetRootObjectPtr());
		if (LevelInstanceTarget && !LevelInstanceTarget->IsEditing())
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_DragInNonEditingLevelInstance", "Cannot drag into a LevelInstance which is not being edited"));
		}

		if (Payload.Has<FFolderTreeItem>())
		{
			FFolder::FRootObject CommonFolderRootObject;
			TArray<FName> DraggedFolders;
			if (GetFolderNamesFromPayload(Payload, DraggedFolders, CommonFolderRootObject))
			{
				// Iterate over all the folders that have been dragged
				for (const FName& DraggedFolder : DraggedFolders)
				{
					const FName Leaf = FEditorFolderUtils::GetLeafName(DraggedFolder);
					const FName Parent = FEditorFolderUtils::GetParentPath(DraggedFolder);

					if ((CommonFolderRootObject != DestinationRootObject) && FFolder::IsRootObjectValid(CommonFolderRootObject) && FFolder::IsRootObjectValid(DestinationRootObject))
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("SourceName"), FText::FromName(Leaf));
						FText Text = FText::Format(LOCTEXT("CantChangeFolderRoot", "Cannot change {SourceName} folder root"), Args);
						return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
					}

					if (Parent == DestinationPath.GetPath())
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("SourceName"), FText::FromName(Leaf));

						FText Text;
						if (DestinationPath.IsNone())
						{
							Text = FText::Format(LOCTEXT("FolderAlreadyAssignedRoot", "{SourceName} is already assigned to root"), Args);
						}
						else
						{
							Args.Add(TEXT("DestPath"), FText::FromName(DestinationPath.GetPath()));
							Text = FText::Format(LOCTEXT("FolderAlreadyAssigned", "{SourceName} is already assigned to {DestPath}"), Args);
						}

						return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
					}

					const FString DragFolderPath = DraggedFolder.ToString();
					const FString LeafName = Leaf.ToString();
					const FString DstFolderPath = DestinationPath.IsNone() ? FString() : DestinationPath.ToString();
					const FString NewPath = DstFolderPath / LeafName;

					if (FActorFolders::Get().ContainsFolder(*RepresentingWorld, FFolder(DestinationRootObject, FName(*NewPath))))
					{
						// The folder already exists
						FFormatNamedArguments Args;
						Args.Add(TEXT("DragName"), FText::FromString(LeafName));
						return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric,
							FText::Format(LOCTEXT("FolderAlreadyExistsRoot", "A folder called \"{DragName}\" already exists at this level"), Args));
					}
					else if (DragFolderPath == DstFolderPath || DstFolderPath.StartsWith(DragFolderPath + "/"))
					{
						// Cannot drag as a child of itself
						FFormatNamedArguments Args;
						Args.Add(TEXT("FolderPath"), FText::FromName(DraggedFolder));
						return FSceneOutlinerDragValidationInfo(
							ESceneOutlinerDropCompatibility::IncompatibleGeneric,
							FText::Format(LOCTEXT("ChildOfItself", "Cannot move \"{FolderPath}\" to be a child of itself"), Args));
					}
				}
			}
		}

		if (Payload.Has<FActorTreeItem>())
		{
			const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
			// Iterate over all the actors that have been dragged
			for (const TWeakObjectPtr<AActor>& WeakActor : Payload.GetData<TWeakObjectPtr<AActor>>(SceneOutliner::FWeakActorSelector()))
			{
				const AActor* Actor = WeakActor.Get();

				bool bActorContainedInLevelInstance = false;
				if (LevelInstanceSubsystem)
				{
					if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
					{
						if (!ParentLevelInstance->IsEditing())
						{
							return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_RemoveEditingLevelInstance", "Cannot detach from a LevelInstance which is not being edited"));
						}
						bActorContainedInLevelInstance = true;
					}

					if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
					{
						FText Reason;
						if (!LevelInstanceSubsystem->CanMoveActorToLevel(Actor, &Reason))
						{
							return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Reason);
						}
					}
				}

				if (Actor->IsChildActor())
				{
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText::Format(LOCTEXT("Error_AttachChildActor", "Cannot move {0} as it is a child actor."), FText::FromString(Actor->GetActorLabel())));
				}
				else if ((Actor->GetFolderRootObject() != DestinationRootObject) && FFolder::IsRootObjectValid(Actor->GetFolderRootObject()) && FFolder::IsRootObjectValid(DestinationRootObject))
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("SourceName"), FText::FromString(Actor->GetActorLabel()));
					FText Text = FText::Format(LOCTEXT("CantChangeActorRoot", "Cannot change {SourceName} folder root"), Args);
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
				}
				else if (Actor->GetFolder() == DestinationPath && !Actor->GetSceneOutlinerParent() && !bActorContainedInLevelInstance)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("SourceName"), FText::FromString(Actor->GetActorLabel()));

					FText Text;
					if (DestinationPath.IsNone())
					{
						Text = FText::Format(LOCTEXT("FolderAlreadyAssignedRoot", "{SourceName} is already assigned to root"), Args);
					}
					else
					{
						Args.Add(TEXT("DestPath"), FText::FromName(DestinationPath.GetPath()));
						Text = FText::Format(LOCTEXT("FolderAlreadyAssigned", "{SourceName} is already assigned to {DestPath}"), Args);
					}

					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
				}
				else if (Actor->GetSceneOutlinerParent())
				{
					FText DetachErrorMsg;
					if (!Actor->EditorCanDetachFrom(Actor->GetSceneOutlinerParent(), DetachErrorMsg))
					{
						return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, DetachErrorMsg);
					}
				}
			}
		}

		// Everything else is a valid operation
		if (DestinationPath.IsNone())
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, LOCTEXT("MoveToRoot", "Move to root"));
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("DestPath"), FText::FromName(DestinationPath.GetPath()));
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FText::Format(LOCTEXT("MoveInto", "Move into \"{DestPath}\""), Args));
		}
	}
	else if (DropTarget.IsA<FComponentTreeItem>())
	{
		// we don't allow drag and drop on components for now
		return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
	}
	return FSceneOutlinerDragValidationInfo::Invalid();
}

TSharedPtr<FDragDropOperation> FActorMode::CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const
{
	FSceneOutlinerDragDropPayload DraggedObjects(InTreeItems);

	// If the drag contains only actors, we shortcut and create a simple FActorDragDropGraphEdOp rather than an FSceneOutlinerDragDrop composite op.
	if (DraggedObjects.Has<FActorTreeItem>() && !DraggedObjects.Has<FFolderTreeItem>())
	{
		return FActorDragDropGraphEdOp::New(DraggedObjects.GetData<TWeakObjectPtr<AActor>>(SceneOutliner::FWeakActorSelector()));
	}

	TSharedPtr<FSceneOutlinerDragDropOp> OutlinerOp = MakeShareable(new FSceneOutlinerDragDropOp());

	if (DraggedObjects.Has<FActorTreeItem>())
	{
		TSharedPtr<FActorDragDropOp> ActorOperation = MakeShareable(new FActorDragDropGraphEdOp);
		ActorOperation->Init(DraggedObjects.GetData<TWeakObjectPtr<AActor>>(SceneOutliner::FWeakActorSelector()));
		OutlinerOp->AddSubOp(ActorOperation);
	}

	if (DraggedObjects.Has<FFolderTreeItem>())
	{
		FFolder::FRootObject CommonRootObject;
		TArray<FName> DraggedFolders;
		if (GetFolderNamesFromPayload(DraggedObjects, DraggedFolders, CommonRootObject))
		{
			TSharedPtr<FFolderDragDropOp> FolderOperation = MakeShareable(new FFolderDragDropOp);
			FolderOperation->Init(DraggedFolders, RepresentingWorld.Get(), CommonRootObject);
			OutlinerOp->AddSubOp(FolderOperation);
		}
	}
	OutlinerOp->Construct();
	return OutlinerOp;
}

bool FActorMode::ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const
{
	if (Operation.IsOfType<FSceneOutlinerDragDropOp>())
	{
		const auto& OutlinerOp = static_cast<const FSceneOutlinerDragDropOp&>(Operation);
		if (const auto& FolderOp = OutlinerOp.GetSubOp<FFolderDragDropOp>())
		{
			for (const auto& Folder : FolderOp->Folders)
			{
				OutPayload.DraggedItems.Add(SceneOutliner->GetTreeItem(FFolder(FolderOp->RootObject, Folder)));
			}
		}
		if (const auto& ActorOp = OutlinerOp.GetSubOp<FActorDragDropOp>())
		{
			for (const TWeakObjectPtr<AActor>& Actor : ActorOp->Actors)
			{
				if (!Actor.IsValid())
				{
					continue;
				}
				OutPayload.DraggedItems.Add(SceneOutliner->GetTreeItem(Actor.Get()));
			}
		}
		return true;
	}
	else if (Operation.IsOfType<FActorDragDropOp>())
	{
		for (const TWeakObjectPtr<AActor>& Actor : static_cast<const FActorDragDropOp&>(Operation).Actors)
		{
			if (!Actor.IsValid())
			{
				continue;
			}
			OutPayload.DraggedItems.Add(SceneOutliner->GetTreeItem(Actor.Get()));
		}
		return true;
	}

	return false;
}

void FActorMode::OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const
{
	if (const FActorTreeItem* ActorItem = DropTarget.CastTo<FActorTreeItem>())
	{
		AActor* DropActor = ActorItem->Actor.Get();
		if (!DropActor)
		{
			return;
		}

		FMessageLog EditorErrors("EditorErrors");
		EditorErrors.NewPage(LOCTEXT("ActorAttachmentsPageLabel", "Actor attachment"));

		if (ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleMultipleDetach || ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleDetach)
		{
			const FScopedTransaction Transaction(LOCTEXT("UndoAction_DetachActors", "Detach actors"));

			TArray<TWeakObjectPtr<AActor>> DraggedActors = Payload.GetData<TWeakObjectPtr<AActor>>(SceneOutliner::FWeakActorSelector());
			for (const auto& WeakActor : DraggedActors)
			{
				if (auto* DragActor = WeakActor.Get())
				{
					// Detach from parent
					USceneComponent* RootComp = DragActor->GetRootComponent();
					if (RootComp && RootComp->GetAttachParent())
					{
						AActor* OldParent = RootComp->GetAttachParent()->GetOwner();
						// Attachment is persisted on the child so modify both actors for Undo/Redo but do not mark the Parent package dirty
						OldParent->Modify(/*bAlwaysMarkDirty=*/false);
						RootComp->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);

						DragActor->SetFolderPath_Recursively(OldParent->GetFolderPath());
					}
				}
			}
		}
		else if (ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleMultipleAttach || ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleAttach)
		{
			// Show socket chooser if we have sockets to select

			if (ILevelInstanceInterface* TargetLevelInstance = Cast<ILevelInstanceInterface>(DropActor))
			{
				check(TargetLevelInstance->IsEditing());
				const FScopedTransaction Transaction(LOCTEXT("UndoAction_MoveActorsToLevelInstance", "Move actors to LevelInstance"));

				const FFolder DestinationPath = FFolder(FFolder::FRootObject(DropActor));
				auto MoveToDestination = [&DestinationPath](FFolderTreeItem& Item)
				{
					Item.MoveTo(DestinationPath);
				};
				Payload.ForEachItem<FFolderTreeItem>(MoveToDestination);

				// Since target root is directly the Level Instance, clear folder path
				TArray<AActor*> DraggedActors = Payload.GetData<AActor*>(SceneOutliner::FActorSelector());
				for (auto& Actor : DraggedActors)
				{
					Actor->SetFolderPath_Recursively(FName());
				}

				ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
				check(LevelInstanceSubsystem);
				LevelInstanceSubsystem->MoveActorsTo(TargetLevelInstance, DraggedActors);
			}
			else
			{
				auto PerformAttachment = [this](FName SocketName, TWeakObjectPtr<AActor> Parent, const TArray<TWeakObjectPtr<AActor>> NewAttachments)
				{
					AActor* ParentActor = Parent.Get();
					if (ParentActor)
					{
						TArray<TWeakObjectPtr<AActor>> AttachedActors;
						// modify parent and child
						const FScopedTransaction Transaction(LOCTEXT("UndoAction_PerformAttachment", "Attach actors"));

						// Attach each child
						for (auto& Child : NewAttachments)
						{
							AActor* ChildActor = Child.Get();
							if (GEditor->CanParentActors(ParentActor, ChildActor))
							{
								GEditor->ParentActors(ParentActor, ChildActor, SocketName);
								AttachedActors.Add(ChildActor);
							}
						}
						OnActorsAttached(ParentActor, AttachedActors);
					}
				};

				TArray<TWeakObjectPtr<AActor>> DraggedActors = Payload.GetData<TWeakObjectPtr<AActor>>(SceneOutliner::FWeakActorSelector());
				//@TODO: Should create a menu for each component that contains sockets, or have some form of disambiguation within the menu (like a fully qualified path)
				// Instead, we currently only display the sockets on the root component
				USceneComponent* Component = DropActor->GetRootComponent();
				if ((Component != NULL) && (Component->HasAnySockets()))
				{
					// Create the popup
					FSlateApplication::Get().PushMenu(
						SceneOutliner->AsShared(),
						FWidgetPath(),
						SNew(SSocketChooserPopup)
						.SceneComponent(Component)
						.OnSocketChosen_Lambda(PerformAttachment, DropActor, MoveTemp(DraggedActors)),
						FSlateApplication::Get().GetCursorPos(),
						FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
					);
				}
				else
				{
					PerformAttachment(NAME_None, DropActor, MoveTemp(DraggedActors));
				}
			}

		}
		// Report errors
		EditorErrors.Notify(NSLOCTEXT("ActorAttachmentError", "AttachmentsFailed", "Attachments Failed!"));
	}
	else if (DropTarget.IsA<FFolderTreeItem>() || DropTarget.IsA<FWorldTreeItem>() || DropTarget.IsA<FLevelTreeItem>())
	{
		const FFolderTreeItem* FolderItem = DropTarget.CastTo<FFolderTreeItem>();
		const FWorldTreeItem* WorldItem = DropTarget.CastTo<FWorldTreeItem>();
		const FLevelTreeItem* LevelItem = DropTarget.CastTo<FLevelTreeItem>();
		// WorldTreeItem and LevelTreeItem are treated as root folders (path = none), with the difference that LevelTreeItem has a RootObject.
		const FFolder DestinationPath = FolderItem ? FolderItem->GetFolder() : (LevelItem ? FFolder(FFolder::GetOptionalFolderRootObject(LevelItem->Level.Get()).Get(FFolder::GetInvalidRootObject())) : GetWorldDefaultRootFolder());

		const FScopedTransaction Transaction(LOCTEXT("MoveOutlinerItems", "Move World Outliner Items"));

		auto MoveToDestination = [&DestinationPath](FFolderTreeItem& Item)
		{
			Item.MoveTo(DestinationPath);
		};
		Payload.ForEachItem<FFolderTreeItem>(MoveToDestination);

		// Set the folder path on all the dragged actors, and detach any that need to be moved
		if (Payload.Has<FActorTreeItem>())
		{
			TSet<const AActor*> ParentActors;
			TSet<const AActor*> ChildActors;

			TArray<AActor*> MovingActorsToValidRootObject;
			Payload.ForEachItem<FActorTreeItem>([&DestinationPath, &ParentActors, &ChildActors, &MovingActorsToValidRootObject](const FActorTreeItem& ActorItem)
			{
				AActor* Actor = ActorItem.Actor.Get();
				if (Actor)
				{
					// First mark this object as a parent, then set its children's path
					ParentActors.Add(Actor);

					const FFolder SrcFolder = Actor->GetFolder();

					// If the folder root object changes, 1st pass will put actors at root. 2nd pass will set the destination path.
					FName NewPath = (SrcFolder.GetRootObject() == DestinationPath.GetRootObject()) ? DestinationPath.GetPath() : NAME_None;

					Actor->SetFolderPath(NewPath);
					FActorEditorUtils::TraverseActorTree_ParentFirst(Actor, [&](AActor* InActor) {
						ChildActors.Add(InActor);
						InActor->SetFolderPath(NewPath);
						return true;
						}, false);

					if ((Actor->GetFolderRootObject() != DestinationPath.GetRootObject()) && SrcFolder.IsRootObjectPersistentLevel() && (DestinationPath.IsRootObjectValid() && !DestinationPath.IsRootObjectPersistentLevel()))
					{
						MovingActorsToValidRootObject.Add(Actor);
					}
				}
			});

			// Detach parent actors
			for (const AActor* Parent : ParentActors)
			{
				auto* RootComp = Parent->GetRootComponent();

				// We don't detach if it's a child of another that's been dragged
				if (RootComp && RootComp->GetAttachParent() && !ChildActors.Contains(Parent))
				{
					if (AActor* OldParentActor = RootComp->GetAttachParent()->GetOwner())
					{
						// Attachment is persisted on the child so modify both actors for Undo/Redo but do not mark the Parent package dirty
						OldParentActor->Modify(/*bAlwaysMarkDirty=*/false);
					}
					RootComp->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
				}
			}

			auto MoveActorsToLevel = [](const TArray<AActor*>& InActorsToMove, ULevel* InDestLevel, const FName& InDestinationPath)
			{
				// We are moving actors to another level
				const bool bWarnAboutReferences = true;
				const bool bWarnAboutRenaming = true;
				const bool bMoveAllOrFail = true;
				TArray<AActor*> MovedActors;
				if (!EditorLevelUtils::MoveActorsToLevel(InActorsToMove, InDestLevel, bWarnAboutReferences, bWarnAboutRenaming, bMoveAllOrFail, &MovedActors))
				{
					UE_LOGF(LogActorMode, Warning, "Failed to move actors because not all actors could be moved");
				}
				// Once moved, update actors folder path
				for (AActor* Actor : MovedActors)
				{
					Actor->SetFolderPath_Recursively(InDestinationPath);
				}
			};

			if (DestinationPath.IsRootObjectPersistentLevel())
			{
				const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
				check(LevelInstanceSubsystem);
				ULevel* DestinationLevel = RepresentingWorld->PersistentLevel;
				check(DestinationLevel);

				TArray<AActor*> LevelInstanceActorsToMove;
				TArray<AActor*> ActorsToMoveToPersistentLevel;
				Payload.ForEachItem<FActorTreeItem>([LevelInstanceSubsystem, &LevelInstanceActorsToMove, &ActorsToMoveToPersistentLevel](const FActorTreeItem& ActorItem)
				{
					AActor* Actor = ActorItem.Actor.Get();
					if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
					{
						check(ParentLevelInstance->IsEditing());
						LevelInstanceActorsToMove.Add(Actor);
					}
					else
					{
						const FFolder ActorSrcFolder = Actor->GetFolder();
						if (ActorSrcFolder.IsRootObjectValid() && !ActorSrcFolder.IsRootObjectPersistentLevel())
						{
							ActorsToMoveToPersistentLevel.Add(Actor);
						}
					}
				});

				// We are moving actors outside of an editing level instance to a folder (or root) into the persistent level.
				if (LevelInstanceActorsToMove.Num() > 0)
				{
					TArray<AActor*> MovedActors;
					LevelInstanceSubsystem->MoveActorsToLevel(LevelInstanceActorsToMove, DestinationLevel, &MovedActors);
					// Once moved, update actors folder path
					for (AActor* Actor : MovedActors)
					{
						Actor->SetFolderPath_Recursively(DestinationPath.GetPath());
					}
				}
				if (ActorsToMoveToPersistentLevel.Num() > 0)
				{
					MoveActorsToLevel(ActorsToMoveToPersistentLevel, DestinationLevel, DestinationPath.GetPath());
				}
			}
			else if (MovingActorsToValidRootObject.Num())
			{
				if (ILevelInstanceInterface* TargetLevelInstance = Cast<ILevelInstanceInterface>(DestinationPath.GetRootObjectPtr()))
				{
					// We are moving actors inside an editing level instance
					check(TargetLevelInstance->IsEditing());

					ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
					check(LevelInstanceSubsystem);
					TArray<AActor*> MovedActors;
					LevelInstanceSubsystem->MoveActorsTo(TargetLevelInstance, MovingActorsToValidRootObject, &MovedActors);
					// Once moved, update actors folder path
					for (AActor* Actor : MovedActors)
					{
						Actor->SetFolderPath_Recursively(DestinationPath.GetPath());
					}
				}
				else if (ULevel* DestinationLevel = Cast<ULevel>(DestinationPath.GetRootObjectPtr()))
				{
					MoveActorsToLevel(MovingActorsToValidRootObject, DestinationLevel, DestinationPath.GetPath());
				}
			}
		}
	}
}

bool FActorMode::GetFolderNamesFromPayload(const FSceneOutlinerDragDropPayload& InPayload, TArray<FName>& OutFolders, FFolder::FRootObject& OutCommonRootObject) const
{
	return FFolder::GetFolderPathsAndCommonRootObject(InPayload.GetData<FFolder>(SceneOutliner::FFolderPathSelector()), OutFolders, OutCommonRootObject);
}

FFolder FActorMode::GetWorldDefaultRootFolder() const
{
	return FFolder::GetWorldRootFolder(RepresentingWorld.Get());
}


#undef LOCTEXT_NAMESPACE