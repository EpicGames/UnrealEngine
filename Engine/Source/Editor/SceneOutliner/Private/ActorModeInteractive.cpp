// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorModeInteractive.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "ActorTreeItem.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"

FActorModeInteractive::FActorModeInteractive(const FActorModeParams& Params)
	: FActorMode(Params)
{
	USelection::SelectionChangedEvent.AddRaw(this, &FActorModeInteractive::OnLevelSelectionChanged);
	USelection::SelectObjectEvent.AddRaw(this, &FActorModeInteractive::OnLevelSelectionChanged);

	FEditorDelegates::MapChange.AddRaw(this, &FActorModeInteractive::OnMapChange);
	FEditorDelegates::NewCurrentLevel.AddRaw(this, &FActorModeInteractive::OnNewCurrentLevel);

	FCoreDelegates::OnActorLabelChanged.AddRaw(this, &FActorModeInteractive::OnActorLabelChanged);
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FActorModeInteractive::OnObjectsReplaced);
	FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FActorModeInteractive::OnPostLoadMapWithWorld);
	GEngine->OnLevelActorRequestRename().AddRaw(this, &FActorModeInteractive::OnLevelActorRequestsRename);
}

FActorModeInteractive::~FActorModeInteractive()
{
	USelection::SelectionChangedEvent.RemoveAll(this);
	USelection::SelectObjectEvent.RemoveAll(this);

	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);

	FCoreDelegates::OnActorLabelChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);
	GEngine->OnLevelActorRequestRename().RemoveAll(this);
}

void FActorModeInteractive::OnMapChange(uint32 MapFlags)
{
	// Instruct the scene outliner to generate a new hierarchy
	SceneOutliner->FullRefresh();
}

void FActorModeInteractive::OnNewCurrentLevel()
{
	// Instruct the scene outliner to generate a new hierarchy
	SceneOutliner->FullRefresh();
}

void FActorModeInteractive::OnLevelSelectionChanged(UObject* Obj)
{
	const FSceneOutlinerFilterInfo* ShowOnlySelectedActorsFilter = FilterInfoMap.Find(TEXT("ShowOnlySelectedActors"));

	// Since there is no way to know which items were removed/added to a selection, we must force a full refresh to handle this
	if (ShowOnlySelectedActorsFilter && ShowOnlySelectedActorsFilter->IsFilterActive())
	{
		SceneOutliner->FullRefresh();
	}
	// If the SceneOutliner's reentrant flag is set, the selection change has already been handled in the outliner class
	else if (!SceneOutliner->GetIsReentrant())
	{
		SceneOutliner->ClearSelection();
		SceneOutliner->RefreshSelection();

		// Scroll last item into view  - this means if we are multi-selecting, we show newest selection. @TODO Not perfect though
		if (const AActor* LastSelectedActor = GEditor->GetSelectedActors()->GetBottom<AActor>())
		{
			if (bCollapseOutlinerTreeOnNewSelection)
			{
				SceneOutliner->CollapseAll();
			}
			
			if (FSceneOutlinerTreeItemPtr TreeItem = SceneOutliner->GetTreeItem(LastSelectedActor, false))
			{
				// Only scroll if selection framing is enabled
				if(bAlwaysFrameSelection)
				{
					SceneOutliner->ScrollItemIntoView(TreeItem);
				}
			}
			else
			{
				SceneOutliner::ENewItemAction::Type Action = bAlwaysFrameSelection ? SceneOutliner::ENewItemAction::ScrollIntoView : SceneOutliner::ENewItemAction::Select;
				
				SceneOutliner->OnItemAdded(LastSelectedActor, Action);
				
			}
		}
	}
}

void FActorModeInteractive::OnLevelActorRequestsRename(const AActor* Actor)
{
	TWeakPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();

	if(TSharedPtr<ILevelEditor> LevelEditorPin = LevelEditor.Pin())
	{
		/* We want to execute the rename on the most recently used outliner
		 * TODO: Add a way to pop-out the outliner the rename is done on
		 */
		if(SceneOutliner == LevelEditorPin->GetMostRecentlyUsedSceneOutliner().Get())
		{
			const TArray<FSceneOutlinerTreeItemPtr>& SelectedItems = SceneOutliner->GetSelectedItems();
			if (SelectedItems.Num() > 0)
			{
				// Ensure that the item we want to rename is visible in the tree
				FSceneOutlinerTreeItemPtr ItemToRename = SelectedItems[SelectedItems.Num() - 1];
				if (SceneOutliner->CanExecuteRenameRequest(*ItemToRename) && ItemToRename->CanInteract())
				{
					SceneOutliner->SetPendingRenameItem(ItemToRename);
					SceneOutliner->ScrollItemIntoView(ItemToRename);
				}
			}
		}
	}
}

void FActorModeInteractive::OnPostLoadMapWithWorld(UWorld* World)
{
	SceneOutliner->FullRefresh();
}

void FActorModeInteractive::OnActorLabelChanged(AActor* ChangedActor)
{
	if (!ensure(ChangedActor))
	{
		return;
	}

	if (IsActorDisplayable(ChangedActor) && RepresentingWorld.Get() == ChangedActor->GetWorld())
	{
		// Force create the item otherwise the outliner may not be notified of a change to the item if it is filtered out
		if (FSceneOutlinerTreeItemPtr Item = CreateItemFor<FActorTreeItem>(ChangedActor, true))
		{
			SceneOutliner->OnItemLabelChanged(Item);
		}
	}
}

void FActorModeInteractive::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	for (const TPair<UObject*, UObject*>& Pair : ReplacementMap)
	{
		AActor* Actor = Cast<AActor>(Pair.Value);
		if (Actor && RepresentingWorld.Get() == Actor->GetWorld() && IsActorDisplayable(Actor))
		{
			if (FSceneOutlinerTreeItemPtr Item = CreateItemFor<FActorTreeItem>(Actor, true))
			{
				SceneOutliner->OnItemLabelChanged(Item);
			}
		}
	}
}
