// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPickingMode.h"
#include "SSceneOutliner.h"
#include "ActorTreeItem.h"

FActorPickingMode::FActorPickingMode(const FActorModeParams& Params, FOnSceneOutlinerItemPicked OnItemPickedDelegate)
	: FActorMode(Params)
	, OnItemPicked(OnItemPickedDelegate)
{
}

void FActorPickingMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	// In actor picking mode, we fire off the notification to whoever is listening.
	// This may often cause the widget itself to be enqueued for destruction
	auto SelectedItems = SceneOutliner->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		auto FirstItem = SelectedItems[0];
		if (FirstItem->CanInteract())
		{
			OnItemPicked.ExecuteIfBound(FirstItem.ToSharedRef());
		}
	}
}

void FActorPickingMode::OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType)
{
	// In actor picking mode, we check to see if we have a selected actor, and if so, fire
	// off the notification to whoever is listening.  This may often cause the widget itself
	// to be enqueued for destruction
	TArray<FActorTreeItem*> Actors;
	Selection.Get(Actors);
	if (Actors.Num() == 1 && Actors[0])
	{
		// Signal that an actor was selected. We assume it is valid as it won't have been added to ActorsToSelect if not.
		SceneOutliner->SetItemSelection(Actors[0]->AsShared(), true, ESelectInfo::OnKeyPress);
	}
}

void FActorPickingMode::CreateViewContent(FMenuBuilder& MenuBuilder)
{
	FActorMode::BuildWorldPickerMenu(MenuBuilder);
}
