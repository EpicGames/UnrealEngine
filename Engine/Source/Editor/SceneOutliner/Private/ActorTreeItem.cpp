// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorTreeItem.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerDragDrop.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "ActorEditorUtils.h"
#include "ClassIconFinder.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"
#include "Logging/MessageLog.h"
#include "SSocketChooser.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterPinnedActors.h"
#include "ToolMenu.h"
#include "Engine/Level.h"
#include "Settings/EditorStyleSettings.h"

#define LOCTEXT_NAMESPACE "SceneOutliner_ActorTreeItem"

const FSceneOutlinerTreeItemType FActorTreeItem::Type(&IActorBaseTreeItem::Type);

template<>
struct TWidgetTypeTraits<struct SActorTreeLabel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

struct SActorTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SActorTreeLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FActorTreeItem& ActorItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());

		TreeItemPtr = StaticCastSharedRef<FActorTreeItem>(ActorItem.AsShared());
		ActorPtr = ActorItem.Actor;

		HighlightText = SceneOutliner.GetFilterHighlightText();

		const bool bShouldUseMiddleEllipsis = GetDefault<UEditorStyleSettings>()->bEnableMiddleEllipsis;

		TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;

		auto MainContent = SNew(SHorizontalBox)

			// Main actor label
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
				.Text(this, &SActorTreeLabel::GetDisplayText)
				.ToolTipText(this, &SActorTreeLabel::GetTooltipText)
				.HighlightText(HighlightText)
				.ColorAndOpacity(this, &SActorTreeLabel::GetForegroundColorInternal)
				.OnTextCommitted(this, &SActorTreeLabel::OnLabelCommitted)
				.OnVerifyTextChanged(this, &SActorTreeLabel::OnVerifyItemLabelChanged)
				.OnEnterEditingMode(this, &SActorTreeLabel::OnEnterEditingMode)
				.OnExitEditingMode(this, &SActorTreeLabel::OnExitEditingMode)
				.IsSelected(FIsSelected::CreateSP(&InRow, &STableRow<FSceneOutlinerTreeItemPtr>::IsSelectedExclusively))
				.OverflowPolicy(bShouldUseMiddleEllipsis ? ETextOverflowPolicy::MiddleEllipsis : TOptional<ETextOverflowPolicy>())
				.IsReadOnly(this, &SActorTreeLabel::IsReadOnly)
			];

		if (WeakSceneOutliner.Pin()->GetMode()->IsInteractive())
		{
			ActorItem.RenameRequestEvent.BindSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
		}

		ChildSlot
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
				[
					SNew(SBox)
					.WidthOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
					.HeightOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
					[
						SNew(SImage)
						.Image(this, &SActorTreeLabel::GetIcon)
						.ToolTipText(this, &SActorTreeLabel::GetIconTooltip)
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f)
				[
					MainContent
				]
			];
		SetForegroundColor(TAttribute<FSlateColor>::CreateRaw(this, &SActorTreeLabel::GetForegroundColorInternal));
	}

private:
	TWeakPtr<FActorTreeItem> TreeItemPtr;
	TWeakObjectPtr<AActor> ActorPtr;
	TAttribute<FText> HighlightText;
	
	FText GetDisplayText() const
	{
		if (const FSceneOutlinerTreeItemPtr TreeItem = TreeItemPtr.Pin())
		{
			const AActor* Actor = ActorPtr.Get();
			if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
			{
				if (!bInEditingMode)
				{
					FText IsCurrentSuffixText = LevelInstance->GetLoadedLevel() && LevelInstance->GetLoadedLevel()->IsCurrentLevel() ? FText(LOCTEXT("IsCurrentSuffix", " (Current)")) : FText::GetEmpty();
					return FText::Format(LOCTEXT("LevelInstanceDisplay", "{0}{1}"), FText::FromString(TreeItem->GetDisplayString()), IsCurrentSuffixText);
				}
			}
			return FText::FromString(TreeItem->GetDisplayString());
		}

		return FText();
	}

	FText GetTooltipText() const
	{
		if (const FSceneOutlinerTreeItemPtr TreeItem = TreeItemPtr.Pin())
		{
			return FText::FromString(TreeItem->GetDisplayString());
		}

		return FText();
	}

	const FSlateBrush* GetIcon() const
	{
		if (const AActor* Actor = ActorPtr.Get())
		{
			if (WeakSceneOutliner.IsValid())
			{
				FName IconName = Actor->GetCustomIconName();
				if (IconName == NAME_None)
				{
					IconName = Actor->GetClass()->GetFName();
				}

				const FSlateBrush* CachedBrush = WeakSceneOutliner.Pin()->GetCachedIconForClass(IconName);
				if (CachedBrush != nullptr)
				{
					return CachedBrush;
				}
				else
				{

					const FSlateBrush* FoundSlateBrush = FClassIconFinder::FindIconForActor(Actor);
					WeakSceneOutliner.Pin()->CacheIconForClass(IconName, FoundSlateBrush);
					return FoundSlateBrush;
				}
			}
			else
			{
				return nullptr;
			}
		}
		else
		{
			return FSlateIconFinder::FindIconForClass(AActor::StaticClass()).GetOptionalIcon();
		}
	}

	const FSlateBrush* GetIconOverlay() const
	{
		static const FName SequencerActorTag(TEXT("SequencerActor"));
		static const FName SequencerPreviewActorTag(TEXT("SequencerPreviewActor"));

		if (const AActor* Actor = ActorPtr.Get())
		{
			if (Actor->ActorHasTag(SequencerPreviewActorTag))
			{
				return FAppStyle::GetBrush("Sequencer.ReplaceableIconOverlay");
			}
			else if (Actor->ActorHasTag(SequencerActorTag))
			{
				return FAppStyle::GetBrush("Sequencer.SpawnableIconOverlay");
			}
		}
		return nullptr;
	}

	FText GetIconTooltip() const
	{
		auto TreeItem = TreeItemPtr.Pin();
		if (!TreeItem.IsValid())
		{
			return FText();
		}

		FText ToolTipText;
		if (AActor* Actor = ActorPtr.Get())
		{
			ToolTipText = FText::FromString(Actor->GetClass()->GetName());
			if (WeakSceneOutliner.Pin()->GetMode()->IsInteractive())
			{
				USceneComponent* RootComponent = Actor->GetRootComponent();
				if (RootComponent)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("ActorClassName"), ToolTipText);

					if (RootComponent->Mobility == EComponentMobility::Static)
					{
						ToolTipText = FText::Format(LOCTEXT("ComponentMobility_Static", "{ActorClassName} with static mobility"), Args);
					}
					else if (RootComponent->Mobility == EComponentMobility::Stationary)
					{
						ToolTipText = FText::Format(LOCTEXT("ComponentMobility_Stationary", "{ActorClassName} with stationary mobility"), Args);
					}
					else if (RootComponent->Mobility == EComponentMobility::Movable)
					{
						ToolTipText = FText::Format(LOCTEXT("ComponentMobility_Movable", "{ActorClassName} with movable mobility"), Args);
					}
				}
			}
		}

		return ToolTipText;
	}

	/**
	 * Returns the Foreground Color of this widget.
	 * Used to avoid overriding SCompoundWidget::GetForegroundColor for invalidation to work properly.
	 */
	FSlateColor GetForegroundColorInternal() const
	{
		AActor* Actor = ActorPtr.Get();

		// Color LevelInstances differently if they are being edited
		if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
		{
			if (LevelInstance->IsEditing())
			{
				return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
			}
			else if (LevelInstance->IsEditingPropertyOverrides())
			{
				return FAppStyle::Get().GetSlateColor("Colors.AccentBlue");
			}
		}

		auto TreeItem = TreeItemPtr.Pin();
		if (auto BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItem))
		{
			return BaseColor.GetValue();
		}
		
		return FSlateColor::UseForeground();
	}

	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
	{
		return FActorEditorUtils::ValidateActorName(InLabel, OutErrorMessage);
	}

	void OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
	{
		auto* Actor = ActorPtr.Get();
		if (Actor && Actor->IsActorLabelEditable() && !InLabel.ToString().Equals(Actor->GetActorLabel(), ESearchCase::CaseSensitive))
		{
			const FScopedTransaction Transaction(LOCTEXT("SceneOutlinerRenameActorTransaction", "Rename Actor"));
			FActorLabelUtilities::RenameExistingActor(Actor, InLabel.ToString());

			auto Outliner = WeakSceneOutliner.Pin();
			if (Outliner.IsValid())
			{
				Outliner->SetKeyboardFocus();
			}
		}
	}

	void OnEnterEditingMode()
	{
		bInEditingMode = true;
	}

	void OnExitEditingMode()
	{
		bInEditingMode = false;
	}

	bool IsReadOnly() const
	{
		AActor* Actor = ActorPtr.Get();
		return !(Actor && Actor->IsActorLabelEditable() && CanExecuteRenameRequest(*TreeItemPtr.Pin()));
	}

	bool bInEditingMode = false;
};

FActorTreeItem::FActorTreeItem(AActor* InActor)
	// Forward to the other constructor using our type identifier.
	: FActorTreeItem(Type, InActor) {}

FActorTreeItem::FActorTreeItem(FSceneOutlinerTreeItemType TypeIn, AActor* InActor)
	: IActorBaseTreeItem(TypeIn)
	, Actor(InActor)
	, ID(InActor)
{
	check(InActor);

	UpdateDisplayStringInternal();

	Flags.bIsExpanded = InActor->bDefaultOutlinerExpansionState;
	
	bExistsInCurrentWorldAndPIE = GEditor->ObjectsThatExistInEditorWorld.Get(InActor);
}

FSceneOutlinerTreeItemID FActorTreeItem::GetID() const
{
	return ID;
}

FFolder::FRootObject FActorTreeItem::GetRootObject() const
{
	AActor* ActorPtr = Actor.Get();
	return ActorPtr ? ActorPtr->GetFolderRootObject() : nullptr;
}

FString FActorTreeItem::GetDisplayString() const
{
	return DisplayString;
}

bool FActorTreeItem::CanInteract() const
{
	AActor* ActorPtr = Actor.Get();
	if (!ActorPtr || !Flags.bInteractive)
	{
		return false;
	}

	return WeakSceneOutliner.Pin()->GetMode()->CanInteract(*this);
}

TSharedRef<SWidget> FActorTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SActorTreeLabel, *this, Outliner, InRow);
}

bool FActorTreeItem::ShouldShowPinnedState() const
{
	return FLoaderAdapterPinnedActors::SupportsPinning(Actor.Get());
}

bool FActorTreeItem::ShouldShowVisibilityState() const
{
	return true;
}

void FActorTreeItem::OnVisibilityChanged(const bool bNewVisibility)
{
	if (AActor* ActorPtr = Actor.Get())
	{
		// Save the actor to the transaction buffer to support undo/redo, but do
		// not call Modify, as we do not want to dirty the actor's package and
		// we're only editing temporary, transient values
		SaveToTransactionBuffer(ActorPtr, false);
		ActorPtr->SetIsTemporarilyHiddenInEditor(!bNewVisibility);
	}
}

bool FActorTreeItem::GetVisibility() const
{
	// We want deleted actors to appear as if they are visible to minimize visual clutter.
	return !Actor.IsValid() || !Actor->IsTemporarilyHiddenInEditor(true);
}

bool FActorTreeItem::GetPinnedState() const
{
	if (Actor.IsValid())
	{
		if (const UWorld* const World = Actor->GetWorld())
		{
			if (const UWorldPartition* const WorldPartition = World->GetWorldPartition())
			{
				return WorldPartition->IsActorPinned(Actor->GetActorGuid());
			}
		}
	}

	return false;
}

void FActorTreeItem::OnLabelChanged()
{
	UpdateDisplayString();
}

void FActorTreeItem::GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner)
{
	const AActor* ActorPtr = Actor.Get();
	const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(ActorPtr);
	if (LevelInstance && LevelInstance->IsEditing())
	{
		FToolMenuSection& Section = Menu->AddSection("Section");
		FSceneOutlinerMenuHelper::AddMenuEntryCreateFolder(Section, Outliner);
		FSceneOutlinerMenuHelper::AddMenuEntryCleanupFolders(Section, LevelInstance->GetLoadedLevel());
	}
}

FString FActorTreeItem::GetPackageName() const
{
	if (const AActor* ActorPtr = Actor.Get())
	{
		if (UPackage* ActorPackage = ActorPtr->GetSceneOutlinerItemPackage())
		{
			return ActorPackage->GetName();
		}
	}
	
	return IActorBaseTreeItem::GetPackageName();
}

const FGuid& FActorTreeItem::GetGuid() const
{
	static const FGuid InvalidGuid;
	return Actor.IsValid() ? Actor->GetActorGuid() : InvalidGuid;
}

void FActorTreeItem::UpdateDisplayString()
{
	UpdateDisplayStringInternal();
}

void FActorTreeItem::UpdateDisplayStringInternal()
{
	DisplayString = Actor.IsValid() ? Actor->GetActorLabel() : TEXT("None");
}

#undef LOCTEXT_NAMESPACE
