// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentTreeItem.h"
#include "Templates/SharedPointer.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "SceneOutlinerPublicTypes.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "SceneOutlinerDragDrop.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"

#define LOCTEXT_NAMESPACE "SceneOutliner_ComponentTreeItem"

struct SComponentTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SComponentTreeLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FComponentTreeItem& ComponentItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		TreeItemPtr = StaticCastSharedRef<FComponentTreeItem>(ComponentItem.AsShared());
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());

		ComponentPtr = ComponentItem.Component;
		
		HighlightText = SceneOutliner.GetFilterHighlightText();

		TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;

		auto MainContent = SNew(SHorizontalBox)

			// Main actor label
			+ SHorizontalBox::Slot()
			[
				SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
				.Text(this, &SComponentTreeLabel::GetDisplayText)
				.ToolTipText(this, &SComponentTreeLabel::GetTooltipText)
				.HighlightText(HighlightText)
				.ColorAndOpacity(this, &SComponentTreeLabel::GetForegroundColor)
				//.OnTextCommitted(this, &SComponentTreeLabel::OnLabelCommitted)
				//.OnVerifyTextChanged(this, &SComponentTreeLabel::OnVerifyItemLabelChanged)
				.IsReadOnly_Lambda([Item = ComponentItem.AsShared(), this]()
				{
					return !CanExecuteRenameRequest(Item.Get());
				})
				.IsSelected(FIsSelected::CreateSP(&InRow, &STableRow<FSceneOutlinerTreeItemPtr>::IsSelectedExclusively))
			];

		if (WeakSceneOutliner.Pin()->GetMode()->IsInteractive())
		{
			ComponentItem.RenameRequestEvent.BindSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
		}

		ChildSlot
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
				[
					SNew(SBox)
					.WidthOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
					.HeightOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
					[
						SNew(SImage)
						.Image(this, &SComponentTreeLabel::GetIcon)
						.ToolTipText(this, &SComponentTreeLabel::GetIconTooltip)
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 2.0f)
				[
					MainContent
				]
			];

	}

private:
	TWeakPtr<FComponentTreeItem> TreeItemPtr;
	TWeakObjectPtr<UActorComponent> ComponentPtr;
	TAttribute<FText> HighlightText;


	FText GetDisplayText() const
	{
		auto Item = TreeItemPtr.Pin();
		return Item.IsValid() ? FText::FromString(Item->GetDisplayString()) : FText();
	}

	FText GetTypeText() const
	{
		if (const UActorComponent* Component = ComponentPtr.Get())
		{
			return FText::FromName(Component->GetClass()->GetFName());
		}

		return FText();
	}

	EVisibility GetTypeTextVisibility() const
	{
		return HighlightText.Get().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
	}

	FText GetTooltipText() const
	{
		if (const UActorComponent* Component = ComponentPtr.Get())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ID_Name"), LOCTEXT("CustomColumnMode_InternalName", "ID Name"));
			Args.Add(TEXT("Name"), FText::FromString(Component->GetName()));
			return FText::Format(LOCTEXT("ComponentNameTooltip", "{ID_Name}: {Name}"), Args);
		}

		return FText();
	}

	const FSlateBrush* GetIcon() const
	{
		if (const UActorComponent* Component = ComponentPtr.Get())
		{
			if (WeakSceneOutliner.IsValid())
			{
				const FSlateBrush* CachedBrush = WeakSceneOutliner.Pin()->GetCachedIconForClass(Component->GetClass()->GetFName());
				if (CachedBrush != nullptr)
				{
					return CachedBrush;
				}
				else
				{
					const FSlateBrush* FoundSlateBrush = FSlateIconFinder::FindIconBrushForClass(Component->GetClass());
					WeakSceneOutliner.Pin()->CacheIconForClass(Component->GetClass()->GetFName(), const_cast<FSlateBrush*>(FoundSlateBrush));
					return FoundSlateBrush;
				}
			}
			else
			{
				return nullptr;
			}
		}
		else
		{
			return nullptr;
		}
	}

	FText GetIconTooltip() const
	{
		FText ToolTipText;
		return ToolTipText;
	}

	FSlateColor GetForegroundColor() const
	{
		if (auto BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItemPtr.Pin()))
		{
			return BaseColor.GetValue();
		}

		return FSlateColor::UseForeground();
	}
};

const FSceneOutlinerTreeItemType FComponentTreeItem::Type(&ISceneOutlinerTreeItem::Type);

FComponentTreeItem::FComponentTreeItem(UActorComponent* InComponent, bool bInSearchComponentsByActorName)
	: ISceneOutlinerTreeItem(Type)
	, Component(InComponent)
	, ID(InComponent)
	, bSearchComponentsByActorName(bInSearchComponentsByActorName)
{
	AActor* OwningActor = InComponent->GetOwner();
	bExistsInCurrentWorldAndPIE = GEditor->ObjectsThatExistInEditorWorld.Get(OwningActor);

	const FName VariableName = FComponentEditorUtils::FindVariableNameGivenComponentInstance(InComponent);
	const bool bIsArrayVariable = !VariableName.IsNone() && InComponent->GetOwner() != nullptr && FindFProperty<FArrayProperty>(InComponent->GetOwner()->GetClass(), VariableName);

	if (!VariableName.IsNone() && !bIsArrayVariable)
	{
		CachedDisplayString = VariableName.ToString();
	}
}

FSceneOutlinerTreeItemID FComponentTreeItem::GetID() const
{
	return ID;
}

FString FComponentTreeItem::GetDisplayString() const
{
	const UActorComponent* ComponentPtr = Component.Get();
	if (ComponentPtr)
	{
		if (!CachedDisplayString.IsEmpty())
		{
			return CachedDisplayString;
		}
		else
		{
			return ComponentPtr->GetName();
		}
	}
	return LOCTEXT("ComponentLabelForMissingComponent", "(Deleted Component)").ToString();
}

bool FComponentTreeItem::CanInteract() const
{
	UActorComponent* ComponentPtr = Component.Get();
	if (!ComponentPtr || !Flags.bInteractive)
	{
		return false;
	}

	return WeakSceneOutliner.Pin()->GetMode()->CanInteract(*this);
}

TSharedRef<SWidget> FComponentTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SComponentTreeLabel, *this, Outliner, InRow);
}

bool FComponentTreeItem::GetSearchComponentByActorName() const
{
	return bSearchComponentsByActorName;
}

#undef LOCTEXT_NAMESPACE
