// Copyright Epic Games, Inc. All Rights Reserved.

#include "FolderTreeItem.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "ToolMenus.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "SceneOutlinerDragDrop.h"
#include "SSceneOutliner.h"

#include "ActorEditorUtils.h"
#include "EditorActorFolders.h"
#include "EditorFolderUtils.h"

#define LOCTEXT_NAMESPACE "SceneOutliner_FolderTreeItem"

namespace SceneOutliner
{
	bool FFolderPathSelector::operator()(TWeakPtr<ISceneOutlinerTreeItem> Item, FFolder& DataOut) const
	{
		if (FFolderTreeItem* FolderItem = Item.Pin()->CastTo<FFolderTreeItem>())
		{
			if (FolderItem->IsValid())
			{
				DataOut = FolderItem->GetFolder();
				return true;
			}
		}
		return false;
	}
}

const FSceneOutlinerTreeItemType FFolderTreeItem::Type(&ISceneOutlinerTreeItem::Type);

FFolderTreeItem::FFolderTreeItem(const FFolder& InFolder, FSceneOutlinerTreeItemType InType)
	: ISceneOutlinerTreeItem(InType)
	, RootObject(InFolder.GetRootObject())
{
	SetPath(InFolder.GetPath());
}

FFolderTreeItem::FFolderTreeItem(const FFolder& InFolder)
	: ISceneOutlinerTreeItem(Type)
	, RootObject(InFolder.GetRootObject())
{
	SetPath(InFolder.GetPath());
}

FFolderTreeItem::FFolderTreeItem(FName InPath)
	: ISceneOutlinerTreeItem(Type)
	, Path(InPath)
{
	SetPath(InPath);
}

FFolderTreeItem::FFolderTreeItem(FName InPath, FSceneOutlinerTreeItemType InType)
	: ISceneOutlinerTreeItem(InType)
{
	SetPath(InPath);
}

void FFolderTreeItem::SetPath(const FName& InNewPath)
{
	Path = InNewPath;
	LeafName = FEditorFolderUtils::GetLeafName(Path);
}

FSceneOutlinerTreeItemID FFolderTreeItem::GetID() const
{
	FFolderKey Key(Path, RootObject);
	return FSceneOutlinerTreeItemID(Key);
}

FFolder::FRootObject FFolderTreeItem::GetRootObject() const
{
	return RootObject;
}

FString FFolderTreeItem::GetDisplayString() const
{
	return LeafName.ToString();
}

bool FFolderTreeItem::CanInteract() const
{
	return Flags.bInteractive;
}

void FFolderTreeItem::DuplicateHierarchy(TWeakPtr<SSceneOutliner> WeakOutliner)
{
	TSharedPtr<SSceneOutliner> Outliner = WeakOutliner.Pin();

	if (Outliner.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("UndoAction_DuplicateHierarchy", "Duplicate Folder Hierarchy"));
		Outliner->DuplicateFoldersHierarchy();
	}
}

void FFolderTreeItem::GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner)
{
	auto SharedOutliner = StaticCastSharedRef<SSceneOutliner>(Outliner.AsShared());

	const FSlateIcon NewFolderIcon(FAppStyle::GetAppStyleSetName(), "SceneOutliner.NewFolderIcon");
	
	FToolMenuSection& Section = Menu->AddSection("Section");
	Section.AddMenuEntry("CreateSubFolder", LOCTEXT("CreateSubFolder", "Create Sub Folder"), FText(), NewFolderIcon, FUIAction(FExecuteAction::CreateSP(this, &FFolderTreeItem::CreateSubFolder, TWeakPtr<SSceneOutliner>(SharedOutliner))));
	Section.AddMenuEntry("DuplicateFolderHierarchy", LOCTEXT("DuplicateFolderHierarchy", "Duplicate Hierarchy"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(&Outliner, &SSceneOutliner::DuplicateFoldersHierarchy)));
}

#undef LOCTEXT_NAMESPACE
