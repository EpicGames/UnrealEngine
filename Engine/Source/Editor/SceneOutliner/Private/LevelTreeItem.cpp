// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelTreeItem.h"
#include "Engine/Level.h"
#include "LevelUtils.h"
#include "Misc/PackageName.h"
#include "ToolMenus.h"
#include "ISceneOutliner.h"
#include "SSceneOutliner.h"
#include "SceneOutlinerPublicTypes.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/Package.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "SceneOutliner_LevelTreeItem"

const FSceneOutlinerTreeItemType FLevelTreeItem::Type(&ISceneOutlinerTreeItem::Type);

template<>
struct TWidgetTypeTraits<struct SLevelTreeLabel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

struct SLevelTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLevelTreeLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FLevelTreeItem& LevelItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		TreeItemPtr = StaticCastSharedRef<FLevelTreeItem>(LevelItem.AsShared());
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
			[
				SNew(SBox)
				.WidthOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
				.HeightOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
				[
					SNew(SImage)
					.Image(FSlateIconFinder::FindIconBrushForClass(UWorld::StaticClass()))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.ToolTipText(LOCTEXT("LevelIcon_Tooltip", "Level"))
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 2.0f)
			[
				SNew(STextBlock)
				.Text(this, &SLevelTreeLabel::GetDisplayText)
				.HighlightText(SceneOutliner.GetFilterHighlightText())
				.ColorAndOpacity(this, &SLevelTreeLabel::GetForegroundColorInternal)
			]
		];
		SetForegroundColor(TAttribute<FSlateColor>::CreateRaw(this, &SLevelTreeLabel::GetForegroundColorInternal));
	}

private:
	TWeakPtr<FLevelTreeItem> TreeItemPtr;

	FText GetDisplayText() const
	{
		auto Item = TreeItemPtr.Pin();
		return Item.IsValid() ? FText::FromString(Item->GetDisplayString()) : FText();
	}

	/**
	 * Returns the Foreground Color of this widget.
	 * Used to avoid overriding SCompoundWidget::GetForegroundColor for invalidation to work properly.
	 */
	FSlateColor GetForegroundColorInternal() const
	{
		if (auto BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItemPtr.Pin()))
		{
			return BaseColor.GetValue();
		}

		return FSlateColor::UseForeground();
	}
};

FLevelTreeItem::FLevelTreeItem(ULevel* InLevel)
	: ISceneOutlinerTreeItem(Type)
	, Level(InLevel)
	, ID(InLevel)
{
	check(InLevel);
}

FString FLevelTreeItem::GetDisplayString() const
{
	return IsValid() ? FPackageName::GetShortName(Level->GetOutermost()->GetName()) : TEXT("None");
}

TSharedRef<SWidget> FLevelTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SLevelTreeLabel, *this, Outliner, InRow);
}

bool FLevelTreeItem::CanInteract() const
{
	if (!Flags.bInteractive)
	{
		return false;
	}
	return Level.IsValid() ? !FLevelUtils::IsLevelLocked(Level.Get()) : true;
}

void FLevelTreeItem::GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner)
{
	if (Level.IsValid() && Level->IsUsingActorFolders())
	{
		FToolMenuSection& Section = Menu->AddSection("Section");
		FSceneOutlinerMenuHelper::AddMenuEntryCleanupFolders(Section, Level.Get());
	}
}

#undef LOCTEXT_NAMESPACE
