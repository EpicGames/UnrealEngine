// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOutlinerTreeView.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "ISceneOutlinerColumn.h"
#include "ISceneOutlinerMode.h"
#include "SceneOutlinerPublicTypes.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "SceneOutlinerDragDrop.h"
#include "SSceneOutliner.h"
#include "Styling/StyleColors.h"

#include "FolderTreeItem.h"

#define LOCTEXT_NAMESPACE "SSceneOutliner"

static void UpdateOperationDecorator(TWeakPtr<SSceneOutliner> WeakSceneOutlinerPtr, const FDragDropEvent& Event, const FSceneOutlinerDragValidationInfo& ValidationInfo)
{
	if (TSharedPtr<SSceneOutliner> SharedSceneOutliner = WeakSceneOutlinerPtr.Pin())
	{
		if (SharedSceneOutliner->UpdateOperationDecorator(Event, &ValidationInfo))
		{
			return;
		}
	}

	const FSlateBrush* Icon = ValidationInfo.IsValid() ? FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")) : FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

	FDragDropOperation* Operation = Event.GetOperation().Get();
	if (Operation && Operation->IsOfType<FDecoratedDragDropOp>())
	{
		auto* DecoratedOp = static_cast<FDecoratedDragDropOp*>(Operation);
		DecoratedOp->SetToolTip(ValidationInfo.ValidationText, Icon);
	}
}

static void ResetOperationDecorator(TWeakPtr<SSceneOutliner> WeakSceneOutlinerPtr, const FDragDropEvent& Event)
{
	if (TSharedPtr<SSceneOutliner> SharedSceneOutliner = WeakSceneOutlinerPtr.Pin())
	{
		if (SharedSceneOutliner->UpdateOperationDecorator(Event, nullptr))
		{
			return;
		}
	}

	FDragDropOperation* Operation = Event.GetOperation().Get();
	if (Operation)
	{
		if (Operation->IsOfType<FSceneOutlinerDragDropOp>())
		{
			static_cast<FSceneOutlinerDragDropOp*>(Operation)->ResetTooltip();
		}
		else if (Operation->IsOfType<FDecoratedDragDropOp>())
		{
			static_cast<FDecoratedDragDropOp*>(Operation)->ResetToDefaultToolTip();
		}
	}
}

static FReply HandleOnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr<SSceneOutlinerTreeView> Table )
{
	auto TablePtr = Table.Pin();
	if (TablePtr.IsValid() && MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ))
	{
		auto Operation = TablePtr->GetOutlinerPtr().Pin()->CreateDragDropOperation(MouseEvent, TablePtr->GetSelectedItems());

		if (Operation.IsValid())
		{
			return FReply::Handled().BeginDragDrop(Operation.ToSharedRef());
		}
	}

	return FReply::Unhandled();
}

FReply HandleDrop(TSharedPtr<SSceneOutliner> SceneOutlinerPtr, const FDragDropEvent& DragDropEvent, ISceneOutlinerTreeItem& DropTarget, FSceneOutlinerDragValidationInfo& ValidationInfo, bool bApplyDrop = false)
{
	if (!SceneOutlinerPtr.IsValid())
	{
		return FReply::Unhandled();
	}

	// Don't handle this if we're not showing a hierarchy
	const FSharedSceneOutlinerData& SharedData = SceneOutlinerPtr->GetSharedData();
	if (!SharedData.bShowParentTree)
	{
		return FReply::Unhandled();
	}

	// Don't handle this if the scene outliner is not in a mode which supports drag and drop
	if (!SceneOutlinerPtr->CanSupportDragAndDrop())
	{
		return FReply::Unhandled();
	}

	FSceneOutlinerDragDropPayload DraggedObjects(*DragDropEvent.GetOperation());
	// Validate now to make sure we don't doing anything we shouldn't
	if (!SceneOutlinerPtr->ParseDragDrop(DraggedObjects, *DragDropEvent.GetOperation()))
	{
		return FReply::Unhandled();
	}

	ValidationInfo = SceneOutlinerPtr->ValidateDrop(StaticCast<ISceneOutlinerTreeItem&>(DropTarget), DraggedObjects);

	if (!ValidationInfo.IsValid())
	{
		// Return handled here to stop anything else trying to handle it - the operation is invalid as far as we're concerned
		return FReply::Handled();
	}

	if (bApplyDrop)
	{
		SceneOutlinerPtr->OnDropPayload(DropTarget, DraggedObjects, ValidationInfo);
	}

	return FReply::Handled();
}

FReply HandleDropFromWeak(TWeakPtr<SSceneOutliner> SceneOutlinerWeak, const FDragDropEvent& DragDropEvent, FSceneOutlinerDragValidationInfo& ValidationInfo, bool bApplyDrop = false)
{
	const ISceneOutlinerMode* Mode = SceneOutlinerWeak.IsValid() ? SceneOutlinerWeak.Pin()->GetMode() : nullptr;
	FFolder::FRootObject RootObject = Mode ? Mode->GetRootObject() : FFolder::GetInvalidRootObject();
	FFolder RootFolder(RootObject);
	FFolderTreeItem DropTarget(RootFolder);
	return HandleDrop(SceneOutlinerWeak.Pin(), DragDropEvent, DropTarget, ValidationInfo, bApplyDrop);
}

void SSceneOutlinerTreeView::Construct(const SSceneOutlinerTreeView::FArguments& InArgs, TSharedRef<SSceneOutliner> Owner)
{
	SceneOutlinerWeak = Owner;
	STreeView::Construct(InArgs);
}

void SSceneOutlinerTreeView::FlashHighlightOnItem( FSceneOutlinerTreeItemPtr FlashHighlightOnItem )
{
	TSharedPtr< SSceneOutlinerTreeRow > RowWidget = StaticCastSharedPtr< SSceneOutlinerTreeRow >( WidgetGenerator.GetWidgetForItem( FlashHighlightOnItem ) );
	if( RowWidget.IsValid() )
	{
		RowWidget->FlashHighlight();
	}
}

FReply SSceneOutlinerTreeView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	FSceneOutlinerDragValidationInfo ValidationInfo = FSceneOutlinerDragValidationInfo::Invalid();
	auto Reply = HandleDropFromWeak(SceneOutlinerWeak, DragDropEvent, ValidationInfo);
	if (Reply.IsEventHandled())
	{
		UpdateOperationDecorator(SceneOutlinerWeak, DragDropEvent, ValidationInfo);
	}

	return Reply;
}

void SSceneOutlinerTreeView::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (!SceneOutlinerWeak.IsValid())
	{
		return;
	}

	if( SceneOutlinerWeak.Pin()->GetSharedData().bShowParentTree )
	{
		ResetOperationDecorator(SceneOutlinerWeak, DragDropEvent);
	}
}

FReply SSceneOutlinerTreeView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	FSceneOutlinerDragValidationInfo ValidationInfo = FSceneOutlinerDragValidationInfo::Invalid();
	return HandleDropFromWeak(SceneOutlinerWeak, DragDropEvent, ValidationInfo, true);
}

void SSceneOutlinerTreeView::Private_UpdateParentHighlights()
{
	this->ClearHighlightedItems();

	// For the Outliner, we want to highlight parent items even if the current selection is not visible (i.e collapsed)
	for( typename TItemSet::TConstIterator SelectedItemIt(SelectedItems); SelectedItemIt; ++SelectedItemIt )
	{
		auto Parent = (*SelectedItemIt)->GetParent();
		while (Parent.IsValid())
		{
			Private_SetItemHighlighted(Parent, true);
			Parent = Parent->GetParent();
		}
	}
}

FReply SSceneOutlinerTreeRow::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	auto ItemPtr = Item.Pin();
	auto SceneOutlinerPtr = SceneOutlinerWeak.Pin();
	if (ItemPtr.IsValid() && SceneOutlinerPtr.IsValid())
	{
		FSceneOutlinerDragValidationInfo ValidationInfo = FSceneOutlinerDragValidationInfo::Invalid();
		return HandleDrop(SceneOutlinerPtr, DragDropEvent, *ItemPtr, ValidationInfo, true);
	}

	return FReply::Unhandled();
}

void SSceneOutlinerTreeRow::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	auto ItemPtr = Item.Pin();
	auto SceneOutlinerPtr = SceneOutlinerWeak.Pin();
	if (ItemPtr.IsValid() && SceneOutlinerPtr.IsValid())
	{
		FSceneOutlinerDragValidationInfo ValidationInfo = FSceneOutlinerDragValidationInfo::Invalid();

		FReply Reply = HandleDrop(SceneOutlinerPtr, DragDropEvent, *ItemPtr, ValidationInfo, false);
		if (Reply.IsEventHandled())
		{
			UpdateOperationDecorator(SceneOutlinerPtr, DragDropEvent, ValidationInfo);
		}
	}
}

void SSceneOutlinerTreeRow::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	ResetOperationDecorator(SceneOutlinerWeak, DragDropEvent);
}

FReply SSceneOutlinerTreeRow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	auto SceneOutlinerPtr = SceneOutlinerWeak.Pin();
	if (SSceneOutliner* SceneOutliner = SceneOutlinerPtr.Get())
	{
		if (const auto* ItemPtr = Item.Pin().Get())
		{
			return SceneOutliner->OnDragOverItem(DragDropEvent, *ItemPtr);
		}
		return FReply::Unhandled();
	}

	return FReply::Handled();
}

FReply SSceneOutlinerTreeRow::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	auto ItemPtr = Item.Pin();
	if (ItemPtr.IsValid() && ItemPtr->CanInteract())
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			FReply Reply = SMultiColumnTableRow<FSceneOutlinerTreeItemPtr>::OnMouseButtonDown( MyGeometry, MouseEvent );

			if (SceneOutlinerWeak.Pin()->CanSupportDragAndDrop())
			{
				return Reply.DetectDrag( SharedThis(this) , EKeys::LeftMouseButton );
			}

			return Reply.PreventThrottling();
		}
	}

	return FReply::Handled();
}

FReply SSceneOutlinerTreeRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	auto ItemPtr = Item.Pin();
	// We don't to change the selection when it is a left click since this was handle in the on mouse down
	if (ItemPtr.IsValid() && ItemPtr->CanInteract())
	{
		return SMultiColumnTableRow<FSceneOutlinerTreeItemPtr>::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

	return FReply::Handled();
}

FReply SSceneOutlinerTreeRow::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	auto ItemPtr = Item.Pin();
	// We don't want to act on double click on an item that can't be interacted with
	if (ItemPtr.IsValid() && ItemPtr->CanInteract())
	{
		return SMultiColumnTableRow<FSceneOutlinerTreeItemPtr>::OnMouseButtonDoubleClick(MyGeometry, MouseEvent);
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SSceneOutlinerTreeRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	auto ItemPtr = Item.Pin();
	if (!ItemPtr.IsValid())
	{
		return SNullWidget::NullWidget;
	}


	auto Outliner = SceneOutlinerWeak.Pin();
	check(Outliner.IsValid());

	// Create the widget for this item
	TSharedRef<SWidget> NewItemWidget = SNullWidget::NullWidget;

	auto Column = Outliner->GetColumns().FindRef(ColumnName);
	if (Column.IsValid())
	{
		NewItemWidget = Column->ConstructRowWidget(ItemPtr.ToSharedRef(), *this);
	}

	if( ColumnName == Outliner->GetPrimaryColumnName() )
	{
		// The first column gets the tree expansion arrow for this row
		return SNew(SBox)
			.MinDesiredHeight(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::RowHeight()))
			[
				SNew( SHorizontalBox )

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(6.f, 0.f, 0.f, 0.f)
				[
					SNew( SExpanderArrow, SharedThis(this) ).IndentAmount(12)
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					NewItemWidget
				]
			];
	}
	else
	{
		// Other columns just get widget content -- no expansion arrow needed
		return NewItemWidget;
	}
}

void SSceneOutlinerTreeRow::Construct( const FArguments& InArgs, const TSharedRef<SSceneOutlinerTreeView>& OutlinerTreeView, TSharedRef<SSceneOutliner> SceneOutliner )
{
	Item = InArgs._Item->AsShared();
	SceneOutlinerWeak = SceneOutliner;
	LastHighlightInteractionTime = 0.0;

	auto Args = FSuperRowType::FArguments()
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));


	Args.OnDragDetected_Static(HandleOnDragDetected, TWeakPtr<SSceneOutlinerTreeView>(OutlinerTreeView));

	SMultiColumnTableRow<FSceneOutlinerTreeItemPtr>::Construct(Args, OutlinerTreeView);
}

const float SSceneOutlinerTreeRow::HighlightRectLeftOffset = 0.0f;
const float SSceneOutlinerTreeRow::HighlightRectRightOffset = 0.0f;
const float SSceneOutlinerTreeRow::HighlightTargetSpringConstant = 25.0f;
const float SSceneOutlinerTreeRow::HighlightTargetEffectDuration = 0.5f;
const float SSceneOutlinerTreeRow::HighlightTargetOpacity = 0.8f;
const float SSceneOutlinerTreeRow::LabelChangedAnimOffsetPercent = 0.2f;

void SSceneOutlinerTreeRow::FlashHighlight()
{
	LastHighlightInteractionTime = FSlateApplication::Get().GetCurrentTime();
	Invalidate(EInvalidateWidgetReason::PaintAndVolatility);
}

bool SSceneOutlinerTreeRow::ComputeVolatility() const
{
	const double CurrentTime = FSlateApplication::Get().GetCurrentTime();
	const float TimeSinceHighlightInteraction = (float)( CurrentTime - LastHighlightInteractionTime );
	return TimeSinceHighlightInteraction <= HighlightTargetEffectDuration;
}

void SSceneOutlinerTreeRow::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Call parent implementation.
	SCompoundWidget::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );

	// Update highlight 'target' effect
	{
		const float HighlightLeftX = HighlightRectLeftOffset;
		const float HighlightRightX = HighlightRectRightOffset + AllottedGeometry.GetLocalSize().X;

		HighlightTargetLeftSpring.SetTarget( HighlightLeftX );
		HighlightTargetRightSpring.SetTarget( HighlightRightX );

		float TimeSinceHighlightInteraction = (float)( InCurrentTime - LastHighlightInteractionTime );
		if( TimeSinceHighlightInteraction <= HighlightTargetEffectDuration)
		{
			HighlightTargetLeftSpring.Tick( InDeltaTime );
			HighlightTargetRightSpring.Tick( InDeltaTime );
		}
		else if (const float LastTimeSinceHighlightInteraction = (float)( (InCurrentTime - InDeltaTime) - LastHighlightInteractionTime );
			LastTimeSinceHighlightInteraction <= HighlightTargetEffectDuration)
		{
			// We only want to invalidate once after the highlight animation is done, so if it was shown last frame, we recompute volatility
			Invalidate(EInvalidateWidgetReason::PaintAndVolatility);
		}
	}
}

int32 SSceneOutlinerTreeRow::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 StartLayer = SMultiColumnTableRow::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	const int32 TextLayer = 1;

	// See if a disabled effect should be used
	bool bEnabled = ShouldBeEnabled( bParentEnabled );
	ESlateDrawEffect DrawEffects = (bEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const double CurrentTime = FSlateApplication::Get().GetCurrentTime();

	// Draw highlight targeting effect
	const float TimeSinceHighlightInteraction = (float)( CurrentTime - LastHighlightInteractionTime );
	if( TimeSinceHighlightInteraction <= HighlightTargetEffectDuration )
	{

		// Compute animation progress
		float EffectAlpha = FMath::Clamp( TimeSinceHighlightInteraction / HighlightTargetEffectDuration, 0.0f, 1.0f );
		EffectAlpha = 1.0f - EffectAlpha * EffectAlpha;  // Inverse square falloff (looks nicer!)

		// Apply extra opacity falloff when dehighlighting
		float EffectOpacity = EffectAlpha;

		// Figure out a universally visible highlight color.
		FLinearColor HighlightTargetColorAndOpacity = ( (FLinearColor::White - GetColorAndOpacity())*0.5f + FLinearColor(+0.4f, +0.1f, -0.2f)) * InWidgetStyle.GetColorAndOpacityTint();
		HighlightTargetColorAndOpacity.A = HighlightTargetOpacity * EffectOpacity * 255.0f;

		// Compute the bounds offset of the highlight target from where the highlight target spring
		// extents currently lie.  This is used to "grow" or "shrink" the highlight as needed.
		const float LabelChangedAnimOffset = LabelChangedAnimOffsetPercent * AllottedGeometry.GetLocalSize().Y;

		// Choose an offset amount depending on whether we're highlighting, or clearing highlight
		const float EffectOffset = EffectAlpha * LabelChangedAnimOffset;

		const float HighlightLeftX = HighlightTargetLeftSpring.GetPosition() - EffectOffset;
		const float HighlightRightX = HighlightTargetRightSpring.GetPosition() + EffectOffset;
		const float HighlightTopY = 0.0f - LabelChangedAnimOffset;
		const float HighlightBottomY = AllottedGeometry.GetLocalSize().Y + EffectOffset;

		const FVector2D DrawPosition = FVector2D( HighlightLeftX, HighlightTopY );
		const FVector2D DrawSize = FVector2D( HighlightRightX - HighlightLeftX, HighlightBottomY - HighlightTopY );

		const FSlateBrush* StyleInfo = FAppStyle::GetBrush("SceneOutliner.ChangedItemHighlight");

		// NOTE: We rely on scissor clipping for the highlight rectangle
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + TextLayer,
			AllottedGeometry.ToPaintGeometry( DrawSize, FSlateLayoutTransform(DrawPosition) ),	// Position, Size, Scale
			StyleInfo,													// Style
			DrawEffects,												// Effects to use
			HighlightTargetColorAndOpacity );							// Color
	}

	return FMath::Max(StartLayer, LayerId + TextLayer);
}

void SSceneOutlinerPinnedTreeRow::Construct(const FArguments& InArgs, const TSharedRef<SSceneOutlinerTreeView>& OutlinerTreeView, TSharedRef<SSceneOutliner> SceneOutliner)
{
	Item = InArgs._Item->AsShared();
	SceneOutlinerWeak = SceneOutliner;
	OutlinerTreeViewWeak = OutlinerTreeView;

	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));

	Args.OnDragDetected_Static(HandleOnDragDetected, TWeakPtr<SSceneOutlinerTreeView>(OutlinerTreeView));

	SMultiColumnTableRow<FSceneOutlinerTreeItemPtr>::Construct(Args, OutlinerTreeView);

}

TSharedRef<SWidget> SSceneOutlinerPinnedTreeRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin();
	if (!ItemPtr.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SSceneOutliner> Outliner = SceneOutlinerWeak.Pin();
	check(Outliner.IsValid());

	// Create the widget for this item
	TSharedRef<SWidget> NewItemWidget = SNullWidget::NullWidget;

	TSharedPtr<ISceneOutlinerColumn> Column = Outliner->GetColumns().FindRef(ColumnName);

	if (Column.IsValid())
	{
		// Construct the actual column widget first
		TSharedRef<SWidget> ActualWidget = Column->ConstructRowWidget(ItemPtr.ToSharedRef(), *this);

		if (ColumnName == FSceneOutlinerBuiltInColumnTypes::Label())
		{
			// We add space for the expander arrow for the label widget to make sure the spacing/indentation is consistent with non-pinned rows
			NewItemWidget = SNew(SBox)
				.MinDesiredHeight(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::RowHeight()))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6.f, 0.f, 0.f, 0.f)
					[
						SNew(SExpanderArrow, SharedThis(this)).IndentAmount(12)
						.Visibility(EVisibility::Hidden) // Hidden SExpanderArrow to occupy the same space as non-pinned rows
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						ActualWidget
					]
				];
		}
		else
		{
			if (ActualWidget != SNullWidget::NullWidget)
			{
				// Get the sorted column IDs from the outliner
				TArray<FName> SortedColumnIDs;
				Outliner->GetSortedColumnIDs(SortedColumnIDs);

				// Get the current column and label column index to compare
				int32 ColumnIndex = SortedColumnIDs.Find(ColumnName);
				int32 LabelColumnIndex = SortedColumnIDs.Find(FSceneOutlinerBuiltInColumnTypes::Label());

				// If either of the columns don't exist, this widget does not need to occupy space
				if (ColumnIndex == INDEX_NONE || LabelColumnIndex == INDEX_NONE)
				{
					ActualWidget->SetVisibility(EVisibility::Collapsed);
				}

				// If this column is to the LEFT of the label column, it is hidden but occupies space to ensure proper indentation
				if (ColumnIndex < LabelColumnIndex)
				{
					ActualWidget->SetVisibility(EVisibility::Hidden);
				}
				// If this column is to the RIGHT of the label colum, it should not occupy space
				else
				{
					ActualWidget->SetVisibility(EVisibility::Collapsed);
				}

			}

			NewItemWidget = ActualWidget;
		}
		
	}

	return NewItemWidget;
}

FReply SSceneOutlinerPinnedTreeRow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin();
	TSharedPtr<SSceneOutliner> SceneOutlinerPtr = SceneOutlinerWeak.Pin();
	if (ItemPtr.IsValid() && SceneOutlinerPtr.IsValid())
	{
		FSceneOutlinerDragValidationInfo ValidationInfo = FSceneOutlinerDragValidationInfo::Invalid();
		return HandleDrop(SceneOutlinerPtr, DragDropEvent, *ItemPtr, ValidationInfo, true);
	}

	return FReply::Unhandled();
}

void SSceneOutlinerPinnedTreeRow::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin();
	TSharedPtr<SSceneOutliner> SceneOutlinerPtr = SceneOutlinerWeak.Pin();
	if (ItemPtr.IsValid() && SceneOutlinerPtr.IsValid())
	{
		FSceneOutlinerDragValidationInfo ValidationInfo = FSceneOutlinerDragValidationInfo::Invalid();

		FReply Reply = HandleDrop(SceneOutlinerPtr, DragDropEvent, *ItemPtr, ValidationInfo, false);
		if (Reply.IsEventHandled())
		{
			UpdateOperationDecorator(SceneOutlinerPtr, DragDropEvent, ValidationInfo);
		}
	}
}

void SSceneOutlinerPinnedTreeRow::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	ResetOperationDecorator(SceneOutlinerWeak, DragDropEvent);
}

FReply SSceneOutlinerPinnedTreeRow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<SSceneOutliner> SceneOutlinerPtr = SceneOutlinerWeak.Pin();
	if (SSceneOutliner* SceneOutliner = SceneOutlinerPtr.Get())
	{
		if (const ISceneOutlinerTreeItem* ItemPtr = Item.Pin().Get())
		{
			return SceneOutliner->OnDragOverItem(DragDropEvent, *ItemPtr);
		}
		return FReply::Unhandled();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
