// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneOutliner.h"

#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "EditorFolderUtils.h"
#include "Engine/GameViewportClient.h"
#include "FolderTreeItem.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ISceneOutlinerColumn.h"
#include "ISceneOutlinerMode.h"
#include "Layout/WidgetPath.h"
#include "Modules/ModuleManager.h"
#include "SceneOutlinerConfig.h"
#include "SceneOutlinerFilterBar.h"
#include "SceneOutlinerFilters.h"
#include "SceneOutlinerModule.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "ToolMenus.h"
#include "UObject/PackageReload.h"
#include "Widgets/SInvalidationPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Notifications/SNotificationList.h"

DEFINE_LOG_CATEGORY(LogSceneOutliner);

static float GSceneOutlinerProcessingBudgetPerFrame = 5.0f;
static FAutoConsoleVariableRef CVarGuardBandMultiplier(
	TEXT("SceneOutliner.ProcessingBudgetPerFrame"),
	GSceneOutlinerProcessingBudgetPerFrame,
	TEXT("Maximum time in mZilliseconds to spend processing operations per frame"));


#define LOCTEXT_NAMESPACE "SSceneOutliner"

// The amount of time that must pass before the Scene Outliner will attempt a sort when in PIE/SIE.
#define SCENE_OUTLINER_RESORT_TIMER 1.0f

void SSceneOutliner::Construct(const FArguments& InArgs, const FSceneOutlinerInitializationOptions& InInitOptions)
{
	// Copy over the shared data from the initialization options
	static_cast<FSharedSceneOutlinerData&>(*SharedData) = static_cast<const FSharedSceneOutlinerData&>(InInitOptions);

	// We use the filter collection provided, otherwise we create our own
	Filters = InInitOptions.Filters.IsValid() ? InInitOptions.Filters : MakeShareable(new FSceneOutlinerFilters);

	OutlinerIdentifier = InInitOptions.OutlinerIdentifier;
	
	// Setup the SearchBox
	// Modes can add filters on init so we do the widget creation before initing the mode
	{
		SearchBoxFilter = CreateTextFilter();
		
		FilterTextBoxWidget = SNew(SFilterSearchBox)
		.Visibility( InInitOptions.bShowSearchBox ? EVisibility::Visible : EVisibility::Collapsed )
		.HintText( LOCTEXT( "FilterSearch", "Search..." ) )
		.OnTextChanged( this, &SSceneOutliner::OnFilterTextChanged )
		.OnTextCommitted( this, &SSceneOutliner::OnFilterTextCommitted )
		.DelayChangeNotificationsWhileTyping( true )
		.DelayChangeNotificationsWhileTypingSeconds( 0.2f );
	}
	
	CreateFilterBar(InInitOptions.FilterBarOptions);
	
	check(InInitOptions.ModeFactory.IsBound());
	Mode = InInitOptions.ModeFactory.Execute(this);
	check(Mode);

	if (FilterBar)
	{
		// Defer loading the Filter settings until after the mode is initialized as they can
		// add custom filters and we want those to be loaded/saved along with default filters
		FilterBar->LoadSettings();
	}	

	bProcessingFullRefresh = false;
	bFullRefresh = true;
	bNeedsRefresh = true;
	bNeedsColumRefresh = true;
	bShouldCacheColumnVisibility = true;
	bForceParentItemsExpanded = false;
	bIsReentrant = false;
	bSortDirty = true;
	bSelectionDirty = true;
	SortOutlinerTimer = 0.0f;
	bPendingFocusNextFrame = InInitOptions.bFocusSearchBoxWhenOpened;
	PrimaryColumnName = InInitOptions.PrimaryColumnName;
	
	SortByColumn = PrimaryColumnName;
	SortMode = EColumnSortMode::Ascending;

	UOutlinerConfig::Initialize();
	UOutlinerConfig::Get()->LoadEditorConfig();

	const FSceneOutlinerConfig* SceneOutlinerConfig = GetConstConfig();

	// Load the pinned items visibility from the config file
	if (SceneOutlinerConfig)
	{
		bShouldStackHierarchyHeaders = SceneOutlinerConfig->bShouldStackHierarchyHeaders;
	}

	// @todo outliner: Should probably save this in layout!
	// @todo outliner: Should save spacing for list view in layout

	// Setup Commands
	BindCommands(InInitOptions.CommandList);

	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	SceneOutlinerModule.OnColumnPermissionListChanged().AddSP(this, &SSceneOutliner::OnColumnPermissionListChanged);

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	for (auto& ModeFilterInfo : Mode->GetFilterInfos())
	{
		ModeFilterInfo.Value.InitFilter(Filters);
	}

	SearchBoxFilter->OnChanged().AddSP( this, &SSceneOutliner::FullRefresh );
	Filters->OnChanged().AddSP(this, &SSceneOutliner::FullRefresh);

	HeaderRowWidget =
		SNew( SHeaderRow )
			// Only show the list header if the user configured the outliner for that
			.Visibility(InInitOptions.bShowHeaderRow ? EVisibility::Visible : EVisibility::Collapsed)
			.CanSelectGeneratedColumn(InInitOptions.bCanSelectGeneratedColumns)
			.OnHiddenColumnsListChanged(this, &SSceneOutliner::HandleHiddenColumnsChanged);

	SetupColumns();

	CacheHiddenColumns = TSet(HeaderRowWidget->GetHiddenColumnIds());

	ChildSlot
	[
		VerticalBox
	];

	VerticalBox->AddSlot()
	.AutoHeight()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SMultiLineEditableTextBox)
			.IsReadOnly(true)
			.Visibility_Lambda([this]() { return Mode->HasErrors() ? EVisibility::Visible : EVisibility::Collapsed; })
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
			.BackgroundColor(FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor"))
			.Text(Mode->GetErrorsText())
			.AutoWrapText(true)			
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SButton)
			.OnClicked_Lambda([this] { Mode->RepairErrors(); return FReply::Handled(); })
			.Visibility_Lambda([this]() { return Mode->HasErrors() ? EVisibility::Visible : EVisibility::Collapsed; })
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("SceneOutlinerRepairErrors", "Repair Errors"))
		]
	];

	auto Toolbar = SNew(SHorizontalBox);

	Toolbar->AddSlot()
	.VAlign(VAlign_Center)
	[
		FilterTextBoxWidget.ToSharedRef()
	];

	if (Mode->CanCustomizeToolbar())
	{
		CustomAddToToolbar(Toolbar);
	}
		
	if (Mode->SupportsCreateNewFolder() && InInitOptions.bShowCreateNewFolder)
	{
		Toolbar->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("CreateFolderToolTip", "Create a new folder containing the current selection"))
				.OnClicked(this, &SSceneOutliner::OnCreateFolderClicked)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("SceneOutliner.NewFolderIcon"))
				]
			];
	}


	if (Mode->ShowViewButton())
	{
		// View mode combo button
		Toolbar->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SAssignNew( ViewOptionsComboButton, SComboButton )
			.ComboButtonStyle( FAppStyle::Get(), "SimpleComboButtonWithIcon" ) // Use the tool bar item style for this button
			.OnGetMenuContent( this, &SSceneOutliner::GetViewButtonContent, Mode->ShowFilterOptions())
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image( FAppStyle::Get().GetBrush("Icons.Settings") )
			]
		];
	}

	VerticalBox->AddSlot()
	.AutoHeight()
	.Padding( 8.0f, 8.0f, 8.0f, 4.0f )
	[
		Toolbar
	];

	// Add the FilterBar and the Add Filter button if it exists
	if(FilterBar)
	{
		// Add Filter Menu
		Toolbar->InsertSlot(0)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 2.0f, 0.0f)
		.AutoWidth()
		[
			SSceneOutlinerFilterBar::MakeAddFilterButton(FilterBar.ToSharedRef())
		];

		VerticalBox->AddSlot()
		.AutoHeight()
		.Padding( 0.0f, 0.0f, 0.0f, 4.0f )
		[
			FilterBar.ToSharedRef()
		];
	}

	VerticalBox->AddSlot()
	.FillHeight(1.0)
	[
		SNew( SOverlay )
		+SOverlay::Slot()
		.HAlign( HAlign_Center )
		[
			SNew( STextBlock )
			.Visibility( this, &SSceneOutliner::GetEmptyLabelVisibility )
			.Text( LOCTEXT( "EmptyLabel", "Empty" ) )
			.ColorAndOpacity( FLinearColor( 0.4f, 1.0f, 0.4f ) )
		]

		+SOverlay::Slot()
		[
			SNew(SBorder).BorderImage( FAppStyle::Get().GetBrush("Brushes.Recessed") )
		]

		+SOverlay::Slot()
		[
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
			SNew(SInvalidationPanel)
			.UseDynamicInvalidation(true)
			[
#endif
				SAssignNew( OutlinerTreeView, SSceneOutlinerTreeView, StaticCastSharedRef<SSceneOutliner>(AsShared()) )

				// Determined by the mode
				.SelectionMode( this, &SSceneOutliner::GetSelectionMode )

				// Point the tree to our array of root-level items.  Whenever this changes, we'll call RequestTreeRefresh()
				.TreeItemsSource( &RootTreeItems )

				// Find out when the user selects something in the tree
				.OnSelectionChanged( this, &SSceneOutliner::OnOutlinerTreeSelectionChanged )

				// Called when the user double-clicks with LMB on an item in the list
				.OnMouseButtonDoubleClick( this, &SSceneOutliner::OnOutlinerTreeDoubleClick )

				// Called when the user single-clicks with LMB on an item in the list
				.OnMouseButtonClick( this, &SSceneOutliner::OnOutlinerTreeSingleClick )

				// Called when an item is scrolled into view
				.OnItemScrolledIntoView( this, &SSceneOutliner::OnOutlinerTreeItemScrolledIntoView )

				// Called when an item is expanded or collapsed
				.OnExpansionChanged(this, &SSceneOutliner::OnItemExpansionChanged)

				// Called to child items for any given parent item
				.OnGetChildren( this, &SSceneOutliner::OnGetChildrenForOutlinerTree )

				// Generates the actual widget for a tree item
				.OnGenerateRow( this, &SSceneOutliner::OnGenerateRowForOutlinerTree ) 

				// Generates the actual widget for a pinned tree item
				.OnGeneratePinnedRow(this, &SSceneOutliner::OnGeneratePinnedRowForOutlinerTree)

				// Use the level viewport context menu as the right click menu for tree items
				.OnContextMenuOpening(this, &SSceneOutliner::OnOpenContextMenu)

				// Header for the tree
				.HeaderRow( HeaderRowWidget )

				// Called when an item is expanded or collapsed with the shift-key pressed down
				.OnSetExpansionRecursive(this, &SSceneOutliner::SetItemExpansionRecursive)

				// Make it easier to see hierarchies when there are a lot of items
				.HighlightParentNodesForSelection(true)

				// Show the Hierarchy of actors pinned at the top of the tree view
				.ShouldStackHierarchyHeaders(this, &SSceneOutliner::ShouldStackHierarchyHeaders)

				// Preserve the selection when the selected item is hidden due to a parent collapsing
				.AllowInvisibleItemSelection(true)
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
			]
#endif
		]
	];


	// Bottom panel status bar, if enabled by the mode
	if (Mode->ShowStatusBar())
	{
		VerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::Get().GetBrush("Brushes.Header") )
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(14, 9))
			[
				SNew( STextBlock )
				.Text(this, &SSceneOutliner::GetFilterStatusText)
				.ColorAndOpacity(this, &SSceneOutliner::GetFilterStatusTextColor)
			]
		];
	}

	// Don't allow tool-tips over the header
	HeaderRowWidget->EnableToolTipForceField( true );

	// Populate our data set
	Populate();

	// Register to update when an undo/redo operation has been called to update our list of items
	GEditor->RegisterForUndo( this );

	// Register to be notified when properties are edited
	FCoreUObjectDelegates::OnPackageReloaded.AddRaw(this, &SSceneOutliner::OnAssetReloaded);

	SourceControlHandler = TSharedPtr<FSceneOutlinerSCCHandler>(new FSceneOutlinerSCCHandler());
}

void SSceneOutliner::HandleHiddenColumnsChanged()
{
	if (!bShouldCacheColumnVisibility)
	{
		return;
	}
	
	TSet<FName> HiddenColumns = TSet(HeaderRowWidget->GetHiddenColumnIds());
	FSceneOutlinerConfig* OutlinerConfig = GetMutableConfig();

	if (OutlinerConfig != nullptr)
	{
		TMap<FName, bool> ColumnVisibilities = OutlinerConfig->ColumnVisibilities;

		bool bAnyColumnVisibilityChanged = false;
	
		for (const TPair<FName, TSharedPtr<ISceneOutlinerColumn>>& Pair : Columns)
		{
			const bool bWasColumnVisible = CacheHiddenColumns.Find(Pair.Key) == nullptr;
			const bool bIsColumnVisible = HiddenColumns.Find(Pair.Key) == nullptr;

			// Only update column visibility if it changed
			if (bWasColumnVisible != bIsColumnVisible)
			{
				ColumnVisibilities.FindOrAdd(Pair.Key) = bIsColumnVisible;
				bAnyColumnVisibilityChanged = true;
			}
		}

		// Only call SaveConfig if something actually changed
		if(bAnyColumnVisibilityChanged)
		{
		OutlinerConfig->ColumnVisibilities = ColumnVisibilities;
		SaveConfig();
	}
}
	
	CacheHiddenColumns = MoveTemp(HiddenColumns);
}

void SSceneOutliner::GetSortedColumnIDs(TArray<FName>& OutColumnIDs) const
{
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	TMap<FName, FSceneOutlinerColumnInfo> FilteredColumnMap;
	for(auto It(SharedData->ColumnMap.CreateIterator()); It; ++It)
	{
		if (SceneOutlinerModule.GetColumnPermissionList()->PassesFilter(It.Key()))
		{
			FilteredColumnMap.Add(It.Key(), It.Value());
		}
	}

	// Get a list of sorted columns IDs to create
	OutColumnIDs.Empty();

	OutColumnIDs.Reserve(FilteredColumnMap.Num());
	FilteredColumnMap.GenerateKeyArray(OutColumnIDs);

	OutColumnIDs.Sort([&](const FName& A, const FName& B) {
		return FilteredColumnMap[A].PriorityIndex < FilteredColumnMap[B].PriorityIndex;
		});
}

void SSceneOutliner::AddColumn_Internal(const FName& ColumnId, const FSceneOutlinerColumnInfo& ColumnInfo, const TMap<FName, bool>& ColumnVisibilities, int32 InsertPosition)
{
	if(!HeaderRowWidget)
	{
		return;
	}

	SHeaderRow& HeaderRow = *HeaderRowWidget;

	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	
	// Avoid caching column visibilities while building the columns
	bool const bPreviousShouldCacheColumnVisibility = bShouldCacheColumnVisibility;
	bShouldCacheColumnVisibility = false;
	
	bool bIsVisible = true;

	// If there is a config saved for this column, ignore the default visibility
	if (const bool *ColumnVisibility = ColumnVisibilities.Find(ColumnId))
	{
		bIsVisible = *ColumnVisibility;
	}
	else if (ColumnInfo.Visibility == ESceneOutlinerColumnVisibility::Invisible)
	{
		bIsVisible = false;
	}

	TSharedPtr<ISceneOutlinerColumn> Column;

	if (ColumnInfo.Factory.IsBound())
	{
		Column = ColumnInfo.Factory.Execute(*this);
	}
	else
	{
		Column = SceneOutlinerModule.FactoryColumn(ColumnId, *this);
	}

	if (ensure(Column.IsValid()))
	{
		Columns.Add(ColumnId, Column);

		auto ColumnArgs = Column->ConstructHeaderRowColumn();

		if (Column->SupportsSorting())
		{
			ColumnArgs
				.SortMode(this, &SSceneOutliner::GetColumnSortMode, ColumnId)
				.OnSort(this, &SSceneOutliner::OnColumnSortModeChanged);
		}

		if (ColumnInfo.ColumnLabel.IsSet())
		{
			ColumnArgs.DefaultLabel(ColumnInfo.ColumnLabel);
		}
		else
		{
			if (HeaderRow.GetVisibility() == EVisibility::Visible)
			{
				UE_LOGF(LogSceneOutliner, Log, "Outliner Column %ls does not have a localizable name, please specify one to FSceneOutlinerColumnInfo", *ColumnId.ToString());
			}
				
			ColumnArgs.DefaultLabel(FText::FromName(ColumnId));
		}
			
		if (!ColumnInfo.bCanBeHidden)
		{
			ColumnArgs.ShouldGenerateWidget(true);
		}

		if (ColumnInfo.FillSize.IsSet())
		{
			ColumnArgs.FillWidth(ColumnInfo.FillSize.GetValue());
		}

		if (ColumnInfo.OnGetHeaderContextMenuContent.IsBound())
		{
			ColumnArgs.MenuContent()
			[
				ColumnInfo.OnGetHeaderContextMenuContent.Execute()
			];
		}

		ColumnArgs.HeaderComboVisibility(ColumnInfo.HeaderComboVisibility);

		if(InsertPosition == INDEX_NONE)
		{
			HeaderRow.AddColumn(ColumnArgs);
		}
		else
		{
			HeaderRow.InsertColumn(ColumnArgs, InsertPosition);
		}
		
		HeaderRow.SetShowGeneratedColumn(ColumnId, bIsVisible);
	}

	bShouldCacheColumnVisibility = bPreviousShouldCacheColumnVisibility;
}

void SSceneOutliner::RemoveColumn_Internal(const FName& ColumnId)
{
	Columns.Remove(ColumnId);
	HeaderRowWidget->RemoveColumn(ColumnId);
}

void SSceneOutliner::SetupColumns()
{
	if(!HeaderRowWidget)
	{
		return;
	}
	
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	if (SharedData->ColumnMap.Num() == 0)
	{
		SharedData->UseDefaultColumns();
	}

	TMap<FName, FSceneOutlinerColumnInfo> FilteredColumnMap;
	for (auto It(SharedData->ColumnMap.CreateIterator()); It; ++It)
	{
		if (SceneOutlinerModule.GetColumnPermissionList()->PassesFilter(It.Key()))
		{
			FilteredColumnMap.Add(It.Key(), It.Value());
		}
	}

	Columns.Empty(FilteredColumnMap.Num());
	HeaderRowWidget->ClearColumns();

	TArray<FName> SortedIDs;
	GetSortedColumnIDs(SortedIDs);

	TMap<FName, bool> ColumnVisibilities;
	const FSceneOutlinerConfig* OutlinerConfig = GetConstConfig();

	// Try to load visibility of columns from the config file
	if (OutlinerConfig)
	{
		ColumnVisibilities = OutlinerConfig->ColumnVisibilities;
	}

	for (const FName& ID : SortedIDs)
	{
		AddColumn_Internal(ID, FilteredColumnMap[ID], ColumnVisibilities);
	}
	
	Columns.Shrink();
	bNeedsColumRefresh = false;
}

void SSceneOutliner::RefreshColumns()
{
	bNeedsColumRefresh = true;
}

void SSceneOutliner::OnColumnPermissionListChanged()
{
	RefreshColumns();
	FullRefresh();
}

SSceneOutliner::~SSceneOutliner()
{
	if (FSceneOutlinerModule* SceneOutlinerModule = FModuleManager::GetModulePtr<FSceneOutlinerModule>("SceneOutliner"))
	{
		SceneOutlinerModule->OnColumnPermissionListChanged().RemoveAll(this);
	}

	Mode->GetHierarchy()->OnHierarchyChanged().RemoveAll(this);
	delete Mode;

	if(GEngine)
	{
		GEditor->UnregisterForUndo(this);
	}

	SearchBoxFilter->OnChanged().RemoveAll( this );
	Filters->OnChanged().RemoveAll(this);

	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);
}

void SSceneOutliner::OnItemAdded(const FSceneOutlinerTreeItemID& ItemID, uint8 ActionMask)
{
	NewItemActions.Add(ItemID, ActionMask);
}

FSlateColor SSceneOutliner::GetViewButtonForegroundColor() const
{
	static const FName InvertedForegroundName("InvertedForeground");
	static const FName DefaultForegroundName("DefaultForeground");

	return ViewOptionsComboButton->IsHovered() ? FAppStyle::GetSlateColor(InvertedForegroundName) : FAppStyle::GetSlateColor(DefaultForegroundName);
}
	 
TSharedRef<SWidget> SSceneOutliner::GetViewButtonContent(bool bShowFilters)
{
	// Menu should stay open on selection if filters are not being shown
	TSharedPtr<FExtender> MenuExtender = MakeShared<FExtender>();
	Mode->InitializeViewMenuExtender(MenuExtender);
	FMenuBuilder MenuBuilder(bShowFilters, nullptr, MenuExtender);

	MenuBuilder.BeginSection(SceneOutliner::ExtensionHooks::Hierarchy, LOCTEXT("HierarchyHeading", "Hierarchy"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ExpandAll", "Expand All"),
			LOCTEXT("ExpandAllToolTip", "Expand All Items in the Hierarchy"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSceneOutliner::ExpandAll)));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CollapseAll", "Collapse All"),
			LOCTEXT("CollapseAllToolTip", "Collapse All Items in the Hierarchy"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSceneOutliner::CollapseAll)));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowHierarchy", "Stack Hierarchy Headers"),
			LOCTEXT("ShowHierarchyToolTip", "Toggle pinning of the hierarchy of items at the top of the outliner"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &SSceneOutliner::ToggleStackHierarchyHeaders),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &SSceneOutliner::ShouldStackHierarchyHeaders)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	if (bShowFilters)
	{
		MenuBuilder.BeginSection(SceneOutliner::ExtensionHooks::Show, LOCTEXT("ShowHeading", "Show"));
		{
			// Add mode filters
			for (auto& ModeFilterInfo : Mode->GetFilterInfos())
			{
				ModeFilterInfo.Value.AddMenu(MenuBuilder);
			}
		}
		MenuBuilder.EndSection();
	}
	Mode->CreateViewContent(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

ESelectionMode::Type SSceneOutliner::GetSelectionMode() const
{
	return Mode->GetSelectionMode();
}

void SSceneOutliner::Refresh()
{
	UE_LOGF(LogSceneOutliner, VeryVerbose, "Refresh requested, current refresh delay: %f SceneOutliner = %p", UIRefreshDelay, this);
	bNeedsRefresh = true;
}

void SSceneOutliner::CustomAddToToolbar(TSharedPtr<SHorizontalBox> Toolbar)
{
	Mode->CustomAddToToolbar(Toolbar);
}

void SSceneOutliner::FullRefresh()
{
	UE_LOGF(LogSceneOutliner, Verbose, "Full Refresh");
	bDisableIntermediateSorting = true;
	bFullRefresh = true;
	RefreshSelection();
	Refresh();
}

void SSceneOutliner::RefreshSelection()
{
	bSelectionDirty = true;
}

void SSceneOutliner::Populate()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSceneOutliner::Populate);

	// Block events while we clear out the list
 	TGuardValue<bool> ReentrantGuard(bIsReentrant, true);

	bool bMadeAnySignificantChanges = false;
	if (bFullRefresh)
	{
		// Remember the selected folders
		TArray<TSharedPtr<ISceneOutlinerTreeItem>> SelectedItems = OutlinerTreeView->GetSelectedItems();
		for (const TSharedPtr<ISceneOutlinerTreeItem>& SelectedItem : SelectedItems)
		{
			if (const FFolderTreeItem* FolderItem = SelectedItem->CastTo<FFolderTreeItem>())
			{
				PendingFoldersSelect.Add(FolderItem->GetFolder());
			}
		}

		// Clear the selection here - RepopulateEntireTree will reconstruct it.
		OutlinerTreeView->ClearSelection();

		RepopulateEntireTree();

		bMadeAnySignificantChanges = true;
		bFullRefresh = false;
	}

	const double StartTime = FPlatformTime::Seconds();
	// To avoid checking the time budget for every item.
	const int32 CheckBudgetEveryNthItem = 100;
	int32 Index = 0;
	while (Index < PendingOperations.Num())
	{
		auto& PendingOp = PendingOperations[Index];
		switch (PendingOp.Type)
		{
		case SceneOutliner::FPendingTreeOperation::Added:
			bMadeAnySignificantChanges = AddItemToTree(PendingOp.Item) || bMadeAnySignificantChanges;
			break;

		case SceneOutliner::FPendingTreeOperation::Moved:
			bMadeAnySignificantChanges = true;
			OnItemMoved(PendingOp.Item);
			break;

		case SceneOutliner::FPendingTreeOperation::Removed:
			bMadeAnySignificantChanges = true;
			RemoveItemFromTree(PendingOp.Item);
			break;

		default:
			check(false);
			break;
		}

		++Index;

		if ((Index % CheckBudgetEveryNthItem) == 0)
		{
			const double TimeSpentInMs = (FPlatformTime::Seconds() - StartTime) * 1000.0;
			if (TimeSpentInMs > GSceneOutlinerProcessingBudgetPerFrame)
			{
				UE_LOGF(LogSceneOutliner, Verbose, "Processing out of budget (%.2f ms) : %.2f ms", GSceneOutlinerProcessingBudgetPerFrame, (float)TimeSpentInMs);
				break;
			}
		}
	}
		
	UE_LOGF(LogSceneOutliner, Verbose, "%d Items Processed", Index);
	PendingOperations.RemoveAt(0, Index);

	for (const FFolder& Folder : PendingFoldersSelect)
	{
		if (FSceneOutlinerTreeItemPtr* Item = TreeItemMap.Find(Folder))
		{
			AddToSelection(*Item);
		}
	}
	PendingFoldersSelect.Empty();

	bool bFullRefreshDone = false;

	// Check if we need to sort because we are finished with the populating operations
	bool bFinalSort = false;
	if (PendingOperations.Num() == 0)
	{
		// When done processing a FullRefresh Scroll to First item in selection as it may have been
		// scrolled out of view by the Refresh
		if (bProcessingFullRefresh)
		{
			FSceneOutlinerItemSelection ItemSelection(*OutlinerTreeView);
			if (ItemSelection.Num() > 0)
			{
				FSceneOutlinerTreeItemPtr ItemToScroll = ItemSelection.SelectedItems[0].Pin();
				if (ItemToScroll)
				{
					ScrollItemIntoView(ItemToScroll);
				}
			}

			bFullRefreshDone = true;
		}

		bProcessingFullRefresh = false;
		// We're fully refreshed now.
		NewItemActions.Empty();
		bNeedsRefresh = false;
		if (bDisableIntermediateSorting)
		{
			bDisableIntermediateSorting = false;
			bFinalSort = true;
		}
	}

	// If we are allowing intermediate sorts and met the conditions, or this is the final sort after all ops are complete
	if ((bMadeAnySignificantChanges && !bDisableIntermediateSorting) || bFinalSort)
	{
		RequestSort();
	}

	if (bFullRefreshDone)
	{
		OnFullRefreshDone.Broadcast();
	}
}

bool SSceneOutliner::ShouldShowFolders() const
{
	return Mode->ShouldShowFolders();
}

void SSceneOutliner::EmptyTreeItems()
{
	PendingOperations.Empty();
	TreeItemMap.Reset();
	PendingTreeItemMap.Empty();

	RootTreeItems.Empty();
}

void SSceneOutliner::AddPendingItem(FSceneOutlinerTreeItemPtr Item)
{
	PendingTreeItemMap.Add(Item->GetID(), Item);
	PendingOperations.Emplace(SceneOutliner::FPendingTreeOperation::Added, Item.ToSharedRef());
}

void SSceneOutliner::AddPendingItemAndChildren(FSceneOutlinerTreeItemPtr Item)
{
	if (!Item.IsValid())
	{
		return;
	}

	// Verify that there isn't already a pending operation for this item:
	if (PendingTreeItemMap.Contains(Item->GetID()))
	{
		return;
	}

	AddPendingItem(Item);

	TArray<FSceneOutlinerTreeItemPtr> Children;
	Mode->GetHierarchy()->CreateChildren(Item, Children);
	for (auto& Child : Children)
	{
		AddPendingItem(Child);
	}

	Refresh();
}

void SSceneOutliner::RepopulateEntireTree()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSceneOutliner::RepopulateEntireTree);

	EmptyTreeItems();

	// Rebuild the hierarchy
	Mode->Rebuild();
	Mode->GetHierarchy()->OnHierarchyChanged().AddSP(this, &SSceneOutliner::OnHierarchyChangedEvent);

	// Create all the items which match the filters, parent-child relationships are handled when each item is actually added to the tree

	TArray<FSceneOutlinerTreeItemPtr> Items;
	Mode->GetHierarchy()->CreateItems(Items);

	for (FSceneOutlinerTreeItemPtr& Item : Items)
	{
		AddPendingItem(Item);
	}
	bProcessingFullRefresh = PendingOperations.Num() > 0;

	Refresh();
}

void SSceneOutliner::OnChildRemovedFromParent(ISceneOutlinerTreeItem& Parent)
{
	if (!Parent.GetChildren().Num())
	{
		if (Parent.ShouldRemoveOnceLastChildRemoved())
		{
			// The parent no longer has any children that match the current search terms. Remove it.
			RemoveItemFromTree(Parent.AsShared());
		}
	}
}

void SSceneOutliner::OnItemMoved(const FSceneOutlinerTreeItemRef& ReferenceItem)
{
	// Just remove the item if it no longer matches the filters
	if (!ReferenceItem->Flags.bIsFilteredOut && !PassesAllFilters(ReferenceItem))
	{
		// This will potentially remove any non-matching, empty parents as well
		RemoveItemFromTree(ReferenceItem);
	}
	else if (const FSceneOutlinerTreeItemPtr* ItemInTree = TreeItemMap.Find(ReferenceItem->GetID()))
	{
		FSceneOutlinerTreeItemRef Item = ItemInTree->ToSharedRef();
		// The item still matches the filters (or has children that do)
		// When an item has been asked to move, it will still reside under its old parent
		FSceneOutlinerTreeItemPtr Parent = Item->GetParent();
		if (Parent.IsValid())
		{
			Parent->RemoveChild(Item);
			OnChildRemovedFromParent(*Parent);
		}
		else
		{
			RootTreeItems.Remove(Item);
		}

		Parent = EnsureParentForItem(Item);
		if (Parent.IsValid())
		{
			Parent->AddChild(Item);
			OutlinerTreeView->SetItemExpansion(Parent, true);
		}
		else
		{
			RootTreeItems.Add(Item);
		}
	}
}

FSceneOutlinerTreeItemPtr SSceneOutliner::GetTreeItem(FSceneOutlinerTreeItemID ItemID, bool bIncludePending)
{
	FSceneOutlinerTreeItemPtr Result = TreeItemMap.FindRef(ItemID);
	if (bIncludePending && !Result.IsValid())
	{
		Result = PendingTreeItemMap.FindRef(ItemID);
	}
	return Result;
}

void SSceneOutliner::SetNextUIRefreshDelay(float InDelay)
{
	UE_LOGF(LogSceneOutliner, VeryVerbose, "UI refresh delay set to %f, previously %f Scene Outliner = %p", InDelay, UIRefreshDelay, this);
	UIRefreshDelay = InDelay;
}

void SSceneOutliner::RemoveItemFromTree(FSceneOutlinerTreeItemRef ReferenceItem)
{
	if (const FSceneOutlinerTreeItemPtr* ItemInTree = TreeItemMap.Find(ReferenceItem->GetID()))
	{
		FSceneOutlinerTreeItemRef Item = ItemInTree->ToSharedRef();

		// If the item we are removing is selected, refresh the selection to remove it from there as well
		if(OutlinerTreeView->GetSelectedItems().Contains(Item))
		{
			RefreshSelection();
		}

		auto Parent = Item->GetParent();

		if (Parent.IsValid())
		{
			Parent->RemoveChild(Item);
			OnChildRemovedFromParent(*Parent);
		}
		else
		{
			RootTreeItems.Remove(Item);
		}

		PendingTreeItemMap_Removal.Remove(Item->GetID());

		TreeItemMap.Remove(Item->GetID());

		Mode->OnItemRemoved(Item);
	}
}

FSceneOutlinerTreeItemPtr SSceneOutliner::EnsureParentForItem(FSceneOutlinerTreeItemRef Item)
{
	if (SharedData->bShowParentTree)
	{
		FSceneOutlinerTreeItemPtr Parent = Mode->GetHierarchy()->FindOrCreateParentItem(*Item, TreeItemMap, /*bCreate=*/false);
		if (Parent.IsValid())
		{
			return Parent;
		}
		else
		{
			// Try to find the parent in the pending items
			Parent = Mode->GetHierarchy()->FindOrCreateParentItem(*Item, PendingTreeItemMap, /*bCreate=*/true);				
			if (Parent.IsValid())
			{
				AddUnfilteredItemToTree(Parent.ToSharedRef());

				return Parent;
			}
		}
	}

	return nullptr;
}

bool SSceneOutliner::AddItemToTree(FSceneOutlinerTreeItemRef Item)
{
	const auto ItemID = Item->GetID();

	PendingTreeItemMap.Remove(ItemID);

	// If a tree item already exists that represents the same data or if the item represents invalid data, bail
	if (TreeItemMap.Find(ItemID)  || !Item->IsValid())
	{
		return false;
	}

	// Set the filtered out flag
	Item->Flags.bIsFilteredOut = !PassesAllFilters(Item);

	if (!Item->Flags.bIsFilteredOut)
	{
		AddUnfilteredItemToTree(Item);

		// Check if we need to do anything with this new item
		if (uint8* ActionMask = NewItemActions.Find(ItemID))
		{
			if (*ActionMask & SceneOutliner::ENewItemAction::Select)
			{
				SetItemSelection(Item, true);
			}

			if (*ActionMask & SceneOutliner::ENewItemAction::Rename && CanExecuteRenameRequest(*Item))
			{
				PendingRenameItem = Item;
			}

			if (*ActionMask & (SceneOutliner::ENewItemAction::ScrollIntoView | SceneOutliner::ENewItemAction::Rename))
			{
				ScrollItemIntoView(Item);
			}
		}
	}

	return true;
}

void SSceneOutliner::AddUnfilteredItemToTree(FSceneOutlinerTreeItemRef Item)
{
	FSceneOutlinerTreeItemPtr Parent = EnsureParentForItem(Item);

	const FSceneOutlinerTreeItemID ItemID = Item->GetID();
	if(TreeItemMap.Contains(ItemID))
	{
		UE_LOGF(LogSceneOutliner, Error, "(%d | %ls) already exists in tree.  Dumping map...", GetTypeHash(ItemID), *Item->GetDisplayString() );
		for(TPair<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Entry : TreeItemMap)
		{
			UE_LOGF(LogSceneOutliner, Log, "(%d | %ls)", GetTypeHash(Entry.Key), *Entry.Value->GetDisplayString());
		}

		// this is a fatal error
		check(false);
	}

	TreeItemMap.Add(ItemID, Item);

	if (Parent.IsValid())
	{
		Parent->AddChild(Item);
	}
	else
	{
		RootTreeItems.Add(Item);
	}

	Item->Flags.bIsExpanded = CachedExpansionStateInfo.FindOrAdd(Item->GetID(), Item->Flags.bIsExpanded);
	
	Mode->OnItemAdded(Item);
}

void SSceneOutliner::PopulateSearchStrings(const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings) const
{
	for (const auto& Pair : Columns)
	{
		Pair.Value->PopulateSearchStrings(Item, OutSearchStrings);
	}
}

/** Creates a TextFilter for ISceneOutlinerTreeItem */
TSharedPtr< SceneOutliner::TreeItemTextFilter > SSceneOutliner::CreateTextFilter() const
{
	auto Delegate = SceneOutliner::TreeItemTextFilter::FItemToStringArray::CreateSP( this, &SSceneOutliner::PopulateSearchStrings );

	return MakeShareable( new SceneOutliner::TreeItemTextFilter( Delegate ) );
}

void SSceneOutliner::GetSelectedFolders(TArray<FFolderTreeItem*>& OutFolders) const
{
	return FSceneOutlinerItemSelection(*OutlinerTreeView).Get<FFolderTreeItem>(OutFolders);
}

TSharedPtr<SWidget> SSceneOutliner::OnOpenContextMenu()
{
	return Mode->CreateContextMenu();
}
	
bool SSceneOutliner::Delete_CanExecute()
{
	return Mode->CanDelete();
}
bool SSceneOutliner::Rename_CanExecute()
{
	return Mode->CanRename();
}

void SSceneOutliner::Rename_Execute()
{
	FSceneOutlinerItemSelection ItemSelection(*OutlinerTreeView);
	FSceneOutlinerTreeItemPtr ItemToRename;

	if (Mode->CanRename())
	{
		ItemToRename = OutlinerTreeView->GetSelectedItems()[0];
	}

	if (ItemToRename.IsValid() && CanExecuteRenameRequest(*ItemToRename) && ItemToRename->CanInteract())
	{
		PendingRenameItem = ItemToRename->AsShared();
		ScrollItemIntoView(ItemToRename);
	}
}

bool SSceneOutliner::Cut_CanExecute()
{
	return Mode->CanCut();
}

bool SSceneOutliner::Copy_CanExecute()
{
	return Mode->CanCopy();
}

bool SSceneOutliner::Paste_CanExecute()
{
	return Mode->CanPaste();
}

bool SSceneOutliner::CanSupportDragAndDrop() const
{
	return Mode->CanSupportDragAndDrop();
}

bool SSceneOutliner::CanExecuteRenameRequest(const ISceneOutlinerTreeItem& ItemPtr) const
{
	return Mode->CanRenameItem(ItemPtr);
}

int32 SSceneOutliner::AddFilter(const TSharedRef<FSceneOutlinerFilter>& Filter)
{
	return Filters->Add(Filter);
}

void SSceneOutliner::AddFilterToFilterBar(const TSharedRef<FFilterBase<SceneOutliner::FilterBarType>>& InFilter)
{
	if(FilterBar)
	{
		FilterBar->AddFilter(InFilter);
	}
	
}

void SSceneOutliner::DisableAllFilterBarFilters(bool bRemove)
{
	if(FilterBar)
	{
		bRemove ? FilterBar->RemoveAllFilters() : FilterBar->DisableAllFilters();
	}
}

bool SSceneOutliner::RemoveFilter(const TSharedRef<FSceneOutlinerFilter>& Filter)
{
	return  Filters->Remove(Filter) > 0;
}

int32 SSceneOutliner::AddInteractiveFilter(const TSharedRef<FSceneOutlinerFilter>& Filter)
{
	return InteractiveFilters->Add(Filter);
}

bool SSceneOutliner::RemoveInteractiveFilter(const TSharedRef<FSceneOutlinerFilter>& Filter)
{
	return InteractiveFilters->Remove(Filter) > 0;
}

TSharedPtr<FSceneOutlinerFilter> SSceneOutliner::GetFilterAtIndex(int32 Index)
{
	return StaticCastSharedPtr<FSceneOutlinerFilter>(Filters->GetFilterAtIndex(Index));
}

int32 SSceneOutliner::GetFilterCount() const
{
	return Filters->Num();
}

void SSceneOutliner::AddColumn(FName ColumnId, const FSceneOutlinerColumnInfo& ColumnInfo)
{
	if (!SharedData->ColumnMap.Contains(ColumnId))
	{
		SharedData->ColumnMap.Add(ColumnId, ColumnInfo);

		// Get the new sorted list of columns to make sure this is added in the right position
		TArray<FName> SortedColumnIDs;
		GetSortedColumnIDs(SortedColumnIDs);

		TMap<FName, bool> ColumnVisibilities;
		const FSceneOutlinerConfig* OutlinerConfig = GetConstConfig();
		if(OutlinerConfig)
		{
			ColumnVisibilities = OutlinerConfig->ColumnVisibilities;
		}

		AddColumn_Internal(ColumnId, ColumnInfo, ColumnVisibilities, SortedColumnIDs.Find(ColumnId));
	}
}

void SSceneOutliner::RemoveColumn(FName ColumnId)
{
	if (SharedData->ColumnMap.Contains(ColumnId))
	{
		SharedData->ColumnMap.Remove(ColumnId);
		RemoveColumn_Internal(ColumnId);
	}
}

void SSceneOutliner::SetColumnVisibility(FName ColumnId, bool bIsVisible)
{
	if (Columns.Contains(ColumnId))
	{
		HeaderRowWidget->SetShowGeneratedColumn(ColumnId, bIsVisible);
	}
}

TArray<FName> SSceneOutliner::GetColumnIds() const
{
	TArray<FName> ColumnsName;
	SharedData->ColumnMap.GenerateKeyArray(ColumnsName);
	return ColumnsName;
}

void SSceneOutliner::SetSelection(const TFunctionRef<bool(ISceneOutlinerTreeItem&)> Selector)
{
	TArray<FSceneOutlinerTreeItemPtr> ItemsToAdd;
	for (const auto& Pair : TreeItemMap)
	{
		FSceneOutlinerTreeItemPtr ItemPtr = Pair.Value;
		if (ISceneOutlinerTreeItem* Item = ItemPtr.Get())
		{
			if (Selector(*Item))
			{
				ItemsToAdd.Add(ItemPtr);
			}
		}
	}

	SetItemSelection(ItemsToAdd, true);
}

void SSceneOutliner::SetItemSelection(const TArray<FSceneOutlinerTreeItemPtr>& InItems, bool bSelected, ESelectInfo::Type SelectInfo)
{
	OutlinerTreeView->ClearSelection();
	OutlinerTreeView->SetItemSelection(InItems, bSelected, SelectInfo);
}

void SSceneOutliner::SetItemSelection(const FSceneOutlinerTreeItemPtr& InItem, bool bSelected, ESelectInfo::Type SelectInfo)
{
	OutlinerTreeView->ClearSelection();
	OutlinerTreeView->SetItemSelection(InItem, bSelected, SelectInfo);
}

void SSceneOutliner::AddToSelection(const TArray<FSceneOutlinerTreeItemPtr>& InItems, ESelectInfo::Type SelectInfo)
{
	OutlinerTreeView->SetItemSelection(InItems, true, SelectInfo);
}

void SSceneOutliner::AddToSelection(const FSceneOutlinerTreeItemPtr& InItem, ESelectInfo::Type SelectInfo)
{
	OutlinerTreeView->SetItemSelection(InItem, true, SelectInfo);
}

void SSceneOutliner::RemoveFromSelection(const TArray<FSceneOutlinerTreeItemPtr>& InItems, ESelectInfo::Type SelectInfo)
{
	OutlinerTreeView->SetItemSelection(InItems, false, SelectInfo);
}

void SSceneOutliner::RemoveFromSelection(const FSceneOutlinerTreeItemPtr& InItem, ESelectInfo::Type SelectInfo)
{
	OutlinerTreeView->SetItemSelection(InItem, false, SelectInfo);
}

void SSceneOutliner::AddFolderToSelection(const FName& FolderName)
{
	// Not used (but public) : For backward compatibility, we use Mode->GetRootObject()
	FFolder::FRootObject RootObject = Mode->GetRootObject();
	if (FFolder::IsRootObjectValid(RootObject))
	{
		if (FSceneOutlinerTreeItemPtr* ItemPtr = TreeItemMap.Find(FFolder(RootObject, FolderName)))
		{
			AddToSelection(*ItemPtr);
		}
	}
}

void SSceneOutliner::RemoveFolderFromSelection(const FName& FolderName)
{
	// Not used (but public) : For backward compatibility, we use Mode->GetRootObject()
	FFolder::FRootObject RootObject = Mode->GetRootObject();
	if (FFolder::IsRootObjectValid(RootObject))
	{
		if (FSceneOutlinerTreeItemPtr* ItemPtr = TreeItemMap.Find(FFolder(RootObject, FolderName)))
		{
			RemoveFromSelection(*ItemPtr);
		}
	}
}

void SSceneOutliner::ClearSelection()
{
	if (!bIsReentrant)
	{
		OutlinerTreeView->ClearSelection();
	}
}

void SSceneOutliner::FillFoldersSubMenu(UToolMenu* Menu) const
{
	FFolder::FRootObject TargetRootObject;
	if (!GetCommonRootObjectFromSelection(TargetRootObject))
	{
		return;
	}

	FToolMenuSection& Section = Menu->AddSection("Section");
	Section.AddMenuEntry("CreateNew", LOCTEXT( "CreateNew", "Create New Folder" ), LOCTEXT( "CreateNew_ToolTip", "Move to a new folder" ),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SceneOutliner.NewFolderIcon"), FExecuteAction::CreateSP(const_cast<SSceneOutliner*>(this), &SSceneOutliner::CreateFolder));

	AddMoveToFolderOutliner(Menu);
}

TSharedRef<TSet<FFolder>> SSceneOutliner::GatherInvalidMoveToDestinations() const
{
	// We use a pointer here to save copying the whole array for every invocation of the filter delegate
	TSharedRef<TSet<FFolder>> ExcludedParents(new TSet<FFolder>());

	for (const auto& Item : OutlinerTreeView->GetSelectedItems())
	{
		const FSceneOutlinerTreeItemPtr Parent = Item->GetParent();
		const FFolderTreeItem* ParentFolderItem = Parent.IsValid() ? Parent->CastTo<FFolderTreeItem>() : nullptr;

		if (ParentFolderItem)
		{
			auto FolderHasOtherSubFolders = [&Item](const TWeakPtr<ISceneOutlinerTreeItem>& WeakItem)
			{
				if (WeakItem.Pin()->IsA<FFolderTreeItem>() && WeakItem.Pin() != Item)
				{
					return true;
				}
				return false;
			};

			// Exclude this items direct parent if it is a folder and has no other subfolders we can move to
			bool bFolderHasSubFolders = false;
			for (const TWeakPtr<ISceneOutlinerTreeItem>& ItemPtr : ParentFolderItem->GetChildren())
			{
				if (FolderHasOtherSubFolders(ItemPtr))
				{
					bFolderHasSubFolders = true;
					break;
				}
			}
			
			if (!bFolderHasSubFolders)
			{
				ExcludedParents->Add(ParentFolderItem->GetFolder());
			}
		}
			
		if (FFolderTreeItem* FolderItem = Item->CastTo<FFolderTreeItem>())
		{
			// Cannot move into itself, or any child
			ExcludedParents->Add(FolderItem->GetFolder());
		}
	}

	return ExcludedParents;
}
	
void SSceneOutliner::AddMoveToFolderOutliner(UToolMenu* Menu) const
{
	// We don't show this if the mode does not show folders
	if (!Mode->ShouldShowFolders())
	{
		return;
	}

	FFolder::FRootObject TargetRootObject;
	if (!GetCommonRootObjectFromSelection(TargetRootObject))
	{
		return;
	}

	// Add a mini scene outliner for choosing an existing folder
	FSceneOutlinerInitializationOptions MiniSceneOutlinerInitOptions;
	MiniSceneOutlinerInitOptions.bShowHeaderRow = false;
	MiniSceneOutlinerInitOptions.bFocusSearchBoxWhenOpened = true;
		
	// Don't show any folders that are a child of any of the selected folders
	auto ExcludedParents = GatherInvalidMoveToDestinations();
	if (ExcludedParents->Num())
	{
		// Add a filter if necessary
		auto FilterOutChildFolders = [](const FFolder& Folder, TSharedRef<TSet<FFolder>> ExcludedParents)
		{
			for (const auto& Parent : *ExcludedParents)
			{
				if (Folder == Parent || Folder.IsChildOf(Parent))
				{
					return false;
				}
			}
			return true;
		};

		MiniSceneOutlinerInitOptions.Filters->AddFilterPredicate<FFolderTreeItem>(FFolderTreeItem::FFilterPredicate::CreateStatic(FilterOutChildFolders, ExcludedParents), FSceneOutlinerFilter::EDefaultBehaviour::Pass);
	}

	{
		struct FFilterRoot : public FSceneOutlinerFilter 
		{
			FFilterRoot(const SSceneOutliner& InSceneOutliner) 
				: FSceneOutlinerFilter(FSceneOutlinerFilter::EDefaultBehaviour::Pass) 
				, SceneOutliner(InSceneOutliner)
			{}
			virtual bool PassesFilter(const ISceneOutlinerTreeItem& Item) const override
			{
				FSceneOutlinerTreeItemPtr Parent = SceneOutliner.FindParent(Item);
				// if item has no parent, it is a root item and should be filtered out
				if (!Parent.IsValid())
				{
					return false;
				}
				return DefaultBehaviour == FSceneOutlinerFilter::EDefaultBehaviour::Pass;
			}

			const SSceneOutliner& SceneOutliner;
		};

		// Filter in/out root items according to whether it is valid to move to/from the root
		FSceneOutlinerDragDropPayload DraggedObjects(OutlinerTreeView->GetSelectedItems());
		const bool bMoveToRootValid = Mode->ValidateDrop(FFolderTreeItem(FFolder(TargetRootObject, FFolder::GetEmptyPath())), DraggedObjects).IsValid();
		if (!bMoveToRootValid)
		{
			MiniSceneOutlinerInitOptions.Filters->Add(MakeShared<FFilterRoot>(*this));
		}
	}

	//Let the mode decide how folder selection is handled

	MiniSceneOutlinerInitOptions.ModeFactory = Mode->CreateFolderPickerMode(TargetRootObject);

	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	TSharedRef< SWidget > MiniSceneOutliner =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.MaxHeight(400.0f)
		[
			SNew(SSceneOutliner, MiniSceneOutlinerInitOptions)
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		];

	FToolMenuSection& Section = Menu->AddSection(FName(), LOCTEXT("ExistingFolders", "Existing:"));
	Section.AddEntry(FToolMenuEntry::InitWidget(
		"MiniSceneOutliner",
		MiniSceneOutliner,
		FText::GetEmpty(),
		false));
}

void SSceneOutliner::FillSelectionSubMenu(UToolMenu* Menu) const
{
	FToolMenuSection& Section = Menu->AddSection("Section");
	Section.AddMenuEntry(
		"AddChildrenToSelection",
		LOCTEXT( "AddChildrenToSelection", "Immediate Children" ),
		LOCTEXT( "AddChildrenToSelection_ToolTip", "Select all immediate children of the selected folders" ),
		FSlateIcon(),
		FExecuteAction::CreateSP(const_cast<SSceneOutliner*>(this), &SSceneOutliner::SelectFoldersDescendants, /*bSelectImmediateChildrenOnly=*/ true));
	Section.AddMenuEntry(
		"AddDescendantsToSelection",
		LOCTEXT( "AddDescendantsToSelection", "All Descendants" ),
		LOCTEXT( "AddDescendantsToSelection_ToolTip", "Select all descendants of the selected folders" ),
		FSlateIcon(),
		FExecuteAction::CreateSP(const_cast<SSceneOutliner*>(this), &SSceneOutliner::SelectFoldersDescendants, /*bSelectImmediateChildrenOnly=*/ false));
}

void SSceneOutliner::SelectFoldersDescendants(bool bSelectImmediateChildrenOnly)
{
	TArray<FFolderTreeItem*> SelectedFolders;
	GetSelectedFolders(SelectedFolders);
	OutlinerTreeView->ClearSelection();

	if (SelectedFolders.Num())
	{
		Mode->SelectFoldersDescendants(SelectedFolders, bSelectImmediateChildrenOnly);
	}

	Refresh();
}

void SSceneOutliner::MoveSelectionTo(const FFolder& NewParent)
{
	FSlateApplication::Get().DismissAllMenus();
		
	FFolderTreeItem 	DropTarget(NewParent);
	FSceneOutlinerDragDropPayload 	DraggedObjects(OutlinerTreeView->GetSelectedItems());

	FSceneOutlinerDragValidationInfo Validation = Mode->ValidateDrop(DropTarget, DraggedObjects);
	if (!Validation.IsValid())
	{
		FNotificationInfo Info(Validation.ValidationText);
		Info.ExpireDuration = 3.0f;
		Info.bUseLargeFont = false;
		Info.bFireAndForget = true;
		Info.bUseSuccessFailIcons = true;
		FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("MoveOutlinerItems", "Move Outliner Items"));
	Mode->OnDrop(DropTarget, DraggedObjects, Validation);
}

FReply SSceneOutliner::OnCreateFolderClicked()
{
	CreateFolder();
	return FReply::Handled();
}

void SSceneOutliner::CreateFolder()
{
	const FFolder& NewFolderName = Mode->CreateNewFolder();

	if (!NewFolderName.IsNone())
	{
		// Move any selected folders into the new folder
		auto PreviouslySelectedItems = OutlinerTreeView->GetSelectedItems();
		for (const auto& Item : PreviouslySelectedItems)
		{
			if (FFolderTreeItem* FolderItem = Item->CastTo<FFolderTreeItem>())
			{
				// New folder root object will be identical if whole selection had a common root object.
				// If not, new folder won't have a root object (world), so this check is needed to skip folders with a different root object.
				if (FolderItem->GetRootObject() == NewFolderName.GetRootObject())
				{
					FolderItem->MoveTo(NewFolderName);
				}
			}
		}

		// At this point the new folder will be in our newly added list, so select it and open a rename when it gets refreshed
		NewItemActions.Add(NewFolderName, SceneOutliner::ENewItemAction::Select | SceneOutliner::ENewItemAction::Rename);
		RequestSort();
	}
}

void SSceneOutliner::CopyFoldersBegin()
{
	CacheFoldersEdit.Reset();
	CacheFoldersEditRootObject = FFolder::GetInvalidRootObject();

	TArray<FFolder> SelectedFolders = GetSelection().GetData<FFolder>(SceneOutliner::FFolderPathSelector());
	FFolder::GetFolderPathsAndCommonRootObject(SelectedFolders, CacheFoldersEdit, CacheFoldersEditRootObject);

	FPlatformApplicationMisc::ClipboardPaste(CacheClipboardContents);
}

void SSceneOutliner::CopyFoldersEnd()
{
	if (CacheFoldersEdit.Num() > 0)
	{
		CopyFoldersToClipboard(CacheFoldersEdit, CacheClipboardContents);
		CacheFoldersEdit.Reset();
		CacheFoldersEditRootObject = FFolder::GetInvalidRootObject();
	}
}

void SSceneOutliner::CopyFoldersToClipboard(const TArray<FName>& InFolders, const FString& InPrevClipboardContents)
{
	if (InFolders.Num() > 0)
	{
		// If clipboard paste has changed since we cached it, items must have been cut 
		// so folders need to appended to clipboard contents rather than replacing them.
		FString CurrClipboardContents;
		FPlatformApplicationMisc::ClipboardPaste(CurrClipboardContents);

		FString Buffer = ExportFolderList(InFolders);

		FString* SourceData = (CurrClipboardContents != InPrevClipboardContents ? &CurrClipboardContents : nullptr);

		if (SourceData)
		{
			SourceData->Append(Buffer);
		}
		else
		{
			SourceData = &Buffer;
		}

		// Replace clipboard contents with original plus folders appended
		FPlatformApplicationMisc::ClipboardCopy(**SourceData);
	}
}

bool SSceneOutliner::GetCommonRootObjectFromSelection(FFolder::FRootObject& OutCommonRootObject) const
{
	TOptional<FFolder::FRootObject> CommonRootObject;

	for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : GetSelection().SelectedItems)
	{
		if (auto TreeItem = Item.Pin())
		{
			if (!CommonRootObject.IsSet())
			{
				CommonRootObject = TreeItem->GetRootObject();
			}
			else if (CommonRootObject.GetValue() != TreeItem->GetRootObject())
			{
				OutCommonRootObject = FFolder::GetInvalidRootObject();
				break;
			}
		}
	}
	OutCommonRootObject = CommonRootObject.Get(FFolder::GetInvalidRootObject());
	return FFolder::IsRootObjectValid(OutCommonRootObject);
}

void SSceneOutliner::PasteFoldersBegin(TArray<FName> InFolders)
{
	auto CacheExistingChildrenAction = [this](const FSceneOutlinerTreeItemPtr& Item, const FFolder::FRootObject& InTargetRootObject)
	{
		if (FFolderTreeItem* FolderItem = Item->CastTo<FFolderTreeItem>())
		{
			TArray<FSceneOutlinerTreeItemID> ExistingChildren;
			for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : FolderItem->GetChildren())
			{
				if (Child.IsValid())
				{
					ExistingChildren.Add(Child.Pin()->GetID());
				}
			}

			check(FolderItem->GetRootObject() == InTargetRootObject);
			CachePasteFolderExistingChildrenMap.Add(FolderItem->GetFolder(), ExistingChildren);
		}
	};

	CacheFoldersEdit.Reset();
	CacheFoldersEditRootObject = Mode->GetPasteTargetRootObject();
	CachePasteFolderExistingChildrenMap.Reset();
	PendingFoldersSelect.Reset();

	CacheFoldersEdit = InFolders;

	// Sort folder names so parents appear before children
	CacheFoldersEdit.Sort(FNameLexicalLess());

	// Cache existing children
	for (FName Folder : CacheFoldersEdit)
	{
		if (FSceneOutlinerTreeItemPtr* TreeItem = TreeItemMap.Find(FFolder(CacheFoldersEditRootObject, Folder)))
		{
			CacheExistingChildrenAction(*TreeItem, CacheFoldersEditRootObject);
		}
	}

	// Prepare CacheFolderMap which maps old to new/duplicate folder names
	CacheFolderMap.Reset();
	for (FName Folder : CacheFoldersEdit)
	{
		FName ParentPath = FEditorFolderUtils::GetParentPath(Folder);
		FName LeafName = FEditorFolderUtils::GetLeafName(Folder);
		if (LeafName != TEXT(""))
		{
			if (FName* NewParentPath = CacheFolderMap.Find(ParentPath))
			{
				ParentPath = *NewParentPath;
			}

			FFolder NewFolderPath = Mode->GetFolder(FFolder(CacheFoldersEditRootObject, ParentPath), LeafName);
			CacheFolderMap.Add(Folder, NewFolderPath.GetPath());
		}
	}
}

void SSceneOutliner::PasteFoldersEnd()
{
	ON_SCOPE_EXIT
	{
		CacheFoldersEdit.Reset();
		CacheFoldersEditRootObject = FFolder::GetInvalidRootObject();
		CacheFolderMap.Reset();
		CachePasteFolderExistingChildrenMap.Reset();
	};

	if (CacheFoldersEdit.IsEmpty())
	{
		FullRefresh();
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "PasteItems", "Paste Items"));

	// Create new folder
	TMap<FName, FName> CreatedFolders;
	for (FName Folder : CacheFoldersEdit)
	{
		if (FName* NewFolder = CacheFolderMap.Find(Folder))
		{
			// When using Actor Folder, duplicated actors might already have created the actor folder (when destination rootobject is different)
			if (Mode->CreateFolder(FFolder(CacheFoldersEditRootObject, *NewFolder)))
			{
				CreatedFolders.Add(Folder, *NewFolder);
			}
		}
	}

	// Populate our data set
	Populate();

	// Reparent duplicated items if the folder has been pasted/duplicated
	for (FName OldFolderName : CacheFoldersEdit)
	{
		// Get the new folder that was created from this name
		if (const FName* NewFolderName = CreatedFolders.Find(OldFolderName))
		{
			FFolder NewFolder(CacheFoldersEditRootObject, *NewFolderName);
			FFolder OldFolder(CacheFoldersEditRootObject, OldFolderName);

			if (FSceneOutlinerTreeItemPtr* OldFolderItem = TreeItemMap.Find(OldFolder))
			{
				for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : (*OldFolderItem)->GetChildren())
				{
					// If this child did not exist in the folder before the paste operation, it should be moved to the new folder
					TArray<FSceneOutlinerTreeItemID>* ExistingChildren = CachePasteFolderExistingChildrenMap.Find(OldFolder);

					if (ExistingChildren && !ExistingChildren->Contains(Child.Pin()->GetID()))
					{
						Mode->ReparentItemToFolder(NewFolder, Child.Pin());
					}
				}
			}
			PendingFoldersSelect.Add(NewFolder);
		}
	}

	FullRefresh();
}

void SSceneOutliner::DuplicateFoldersHierarchy()
{
	TFunction<void(const FSceneOutlinerTreeItemPtr&)> RecursiveFolderSelect = [&](const FSceneOutlinerTreeItemPtr& Item)
	{
		if (Item->IsA<FFolderTreeItem>())
		{
			AddToSelection(Item);
		}
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : Item->GetChildren())
		{
			RecursiveFolderSelect(Child.Pin());
		}
	};

	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "DuplicateFoldersHierarchy", "Duplicate Folders Hierarchy"));

	TArray<FFolderTreeItem*> SelectedFolders;
	GetSelectedFolders(SelectedFolders);

	if (SelectedFolders.Num() > 0)
	{
		// Select folder descendants
		SelectFoldersDescendants();

		// Select all sub-folders
		for (FFolderTreeItem* Folder : SelectedFolders)
		{
			RecursiveFolderSelect(Folder->AsShared());
		}

		// Duplicate selected
		Mode->OnDuplicateSelected();
	}
}

FName SSceneOutliner::GetPrimaryColumnName() const
{
	return PrimaryColumnName;
}

void SSceneOutliner::DeleteFoldersBegin()
{
	CacheFoldersDelete.Empty();
	GetSelectedFolders(CacheFoldersDelete);
}

void SSceneOutliner::DeleteFoldersEnd()
{
	struct FMatchFolder
	{
		FMatchFolder(const FFolder& InFolder)
			: Folder(InFolder) {}

		const FFolder Folder;

		bool operator()(const FFolderTreeItem *Entry)
		{
			return Folder == Entry->GetFolder();
		}
	};

	if (CacheFoldersDelete.Num() > 0)
	{
		// Sort in descending order so children will be deleted before parents
		CacheFoldersDelete.Sort([](const FFolderTreeItem& FolderA, const FFolderTreeItem& FolderB)
		{
			return FolderB.GetPath().LexicalLess(FolderA.GetPath());
		});

		for (FFolderTreeItem* FolderItem : CacheFoldersDelete)
		{
			if (FolderItem)
			{
				// Find lowest parent not being deleted, for reparenting children of current folder
				FFolder NewParentPath = FolderItem->GetFolder().GetParent();
				while (!NewParentPath.IsNone() && CacheFoldersDelete.FindByPredicate(FMatchFolder(NewParentPath)))
				{
					NewParentPath = NewParentPath.GetParent();
				}

				FolderItem->Delete(NewParentPath);
			}
		}

		CacheFoldersDelete.Reset();
		FullRefresh();
	}
}

TArray<FName> SSceneOutliner::GetClipboardPasteFolders() const
{
	FString PasteString;
	FPlatformApplicationMisc::ClipboardPaste(PasteString);
	return ImportFolderList(*PasteString);
}

FString SSceneOutliner::ExportFolderList(TArray<FName> InFolders) const
{
	FString Buffer = FString(TEXT("Begin FolderList\n"));

	for (auto& FolderName : InFolders)
	{
		Buffer.Append(FString(TEXT("\tFolder=")) + FolderName.ToString() + FString(TEXT("\n")));
	}

	Buffer += FString(TEXT("End FolderList\n"));

	return Buffer;
}

TArray<FName> SSceneOutliner::ImportFolderList(const FString& InStrBuffer) const
{
	TArray<FName> Folders;

	int32 Index = InStrBuffer.Find(TEXT("Begin FolderList"));
	if (Index != INDEX_NONE)
	{
		FString TmpStr = InStrBuffer.RightChop(Index);
		const TCHAR* Buffer = *TmpStr;

		FString StrLine;
		while (FParse::Line(&Buffer, StrLine))
		{
			const TCHAR* Str = *StrLine;				
			FString FolderName;

			if (FParse::Command(&Str, TEXT("Begin")) && FParse::Command(&Str, TEXT("FolderList")))
			{
				continue;
			}
			else if (FParse::Command(&Str, TEXT("End")) && FParse::Command(&Str, TEXT("FolderList")))
			{
				break;
			}
			else if (FParse::Value(Str, TEXT("Folder="), FolderName))
			{
				Folders.Add(FName(*FolderName));
			}
		}
	}
	return Folders;
}

void SSceneOutliner::ScrollItemIntoView(const FSceneOutlinerTreeItemPtr& Item)
{
	auto Parent = Item->GetParent();
	while (Parent.IsValid())
	{
		OutlinerTreeView->SetItemExpansion(Parent->AsShared(), true);
		Parent = Parent->GetParent();
	}
	OutlinerTreeView->RequestScrollIntoView(Item);
}

void SSceneOutliner::SetItemExpansion(const FSceneOutlinerTreeItemPtr& Item, bool bIsExpanded)
{
	OutlinerTreeView->SetItemExpansion(Item, bIsExpanded);
}

bool SSceneOutliner::IsItemExpanded(const FSceneOutlinerTreeItemPtr& Item) const
{
	return OutlinerTreeView->IsItemExpanded(Item);
}

TSharedRef< ITableRow > SSceneOutliner::OnGenerateRowForOutlinerTree( FSceneOutlinerTreeItemPtr Item, const TSharedRef< STableViewBase >& OwnerTable )
{
	return SNew( SSceneOutlinerTreeRow, OutlinerTreeView.ToSharedRef(), SharedThis(this) ).Item( Item );
}

TSharedRef< ITableRow > SSceneOutliner::OnGeneratePinnedRowForOutlinerTree(FSceneOutlinerTreeItemPtr Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	return SNew(SSceneOutlinerPinnedTreeRow, OutlinerTreeView.ToSharedRef(), SharedThis(this)).Item(Item);
}

void SSceneOutliner::OnGetChildrenForOutlinerTree( FSceneOutlinerTreeItemPtr InParent, TArray< FSceneOutlinerTreeItemPtr >& OutChildren )
{
	if( SharedData->bShowParentTree )
	{
		for (auto& WeakChild : InParent->GetChildren())
		{
			auto Child = WeakChild.Pin();
			// Should never have bogus entries in this list
			check(Child.IsValid());
			OutChildren.Add(Child);
		}

		// If the item needs it's children sorting, do that now
		if (OutChildren.Num() && InParent->Flags.bChildrenRequireSort)
		{
			// Sort the children we returned
			SortItems(OutChildren);

			// Empty out the children and repopulate them in the correct order
			InParent->Children.Empty();
			InParent->Children.Reserve(OutChildren.Num());
			for (auto& Child : OutChildren)
			{
				InParent->Children.Emplace(Child);
			}
				
			// They no longer need sorting
			InParent->Flags.bChildrenRequireSort = false;
		}
	}
}

void SSceneOutliner::OnOutlinerTreeSelectionChanged( FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type SelectInfo )
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	if (!bIsReentrant)
	{
		TGuardValue<bool> ReentrantGuard(bIsReentrant, true);
		Mode->OnItemSelectionChanged(TreeItem, SelectInfo, FSceneOutlinerItemSelection(*OutlinerTreeView));

		OnItemSelectionChanged.Broadcast(TreeItem, SelectInfo);
	}
}

void SSceneOutliner::OnOutlinerTreeDoubleClick( FSceneOutlinerTreeItemPtr TreeItem )
{
	if (!Mode->HasCustomFolderDoubleClick())
	{
		if (TreeItem->IsA<FFolderTreeItem>())
		{
			SetItemExpansion(TreeItem, !IsItemExpanded(TreeItem));
		}
	}

	Mode->OnItemDoubleClick(TreeItem);

	OnDoubleClickOnTreeEvent.Broadcast(TreeItem);
}

void SSceneOutliner::OnOutlinerTreeSingleClick(FSceneOutlinerTreeItemPtr TreeItem) const
{
	Mode->OnItemClicked(TreeItem);
}

void SSceneOutliner::OnOutlinerTreeItemScrolledIntoView( FSceneOutlinerTreeItemPtr TreeItem, const TSharedPtr<ITableRow>& Widget )
{
	if (TreeItem == PendingRenameItem.Pin())
	{
		PendingRenameItem = nullptr;
		TreeItem->RenameRequestEvent.ExecuteIfBound();
	}
}
	
void SSceneOutliner::OnItemExpansionChanged(FSceneOutlinerTreeItemPtr TreeItem, bool bIsExpanded) const
{
	if (bForceParentItemsExpanded)
	{
		return;
	}

	TreeItem->Flags.bIsExpanded = bIsExpanded;
	TreeItem->OnExpansionChanged();

	// Expand any children that are also expanded
	for (auto WeakChild : TreeItem->GetChildren())
	{
		auto Child = WeakChild.Pin();
		if (Child.IsValid() && Child->Flags.bIsExpanded)
		{
			OutlinerTreeView->SetItemExpansion(Child, true);
		}
	}

	// Notify Mode
	CachedExpansionStateInfo.Add(TreeItem->GetID(), TreeItem->Flags.bIsExpanded);
}

void SSceneOutliner::OnHierarchyChangedEvent(FSceneOutlinerHierarchyChangedData Event)
{
	if (Event.Type == FSceneOutlinerHierarchyChangedData::Added)
	{
		for (const auto& TreeItemPtr : Event.Items)
		{
			if(!TreeItemPtr.IsValid())
			{
				continue;
			}

			// If the item doesn't exist in the tree, or is being removed and re-added in the same frame - it is not a duplicate and can be added
			if(!TreeItemMap.Find(TreeItemPtr->GetID()) || PendingTreeItemMap_Removal.Find(TreeItemPtr->GetID()))
			{
				AddPendingItemAndChildren(TreeItemPtr);
				if (Event.ItemActions)
				{
					NewItemActions.Add(TreeItemPtr->GetID(), Event.ItemActions);
				}
			}
		}
	}
	else if (Event.Type == FSceneOutlinerHierarchyChangedData::Removed)
	{
		for (const auto& TreeItemID : Event.ItemIDs)
		{
			FSceneOutlinerTreeItemPtr* Item = TreeItemMap.Find(TreeItemID);
			if (!Item)
			{
				Item = PendingTreeItemMap.Find(TreeItemID);
			}

			if (Item)
			{
				PendingOperations.Emplace(SceneOutliner::FPendingTreeOperation::Removed, Item->ToSharedRef());
				PendingTreeItemMap_Removal.Add(TreeItemID, Item->ToSharedRef());
			}
		}
		
		UE_LOGF(LogSceneOutliner, VeryVerbose, "Refresh requested by FSceneOutlinerHierarchyChangedData::Removed");

		Refresh();
	}
	else if (Event.Type == FSceneOutlinerHierarchyChangedData::Moved)
	{
		for (const auto& TreeItemID : Event.ItemIDs)
		{
			FSceneOutlinerTreeItemPtr* Item = TreeItemMap.Find(TreeItemID);
			if (!Item)
			{
				Item = PendingTreeItemMap.Find(TreeItemID);
			}

			if (Item)
			{
				PendingOperations.Emplace(SceneOutliner::FPendingTreeOperation::Moved, Item->ToSharedRef());
			}
		}

		for (const auto& TreeItemPtr : Event.Items)
		{
			if (TreeItemPtr.IsValid())
			{
				PendingOperations.Emplace(SceneOutliner::FPendingTreeOperation::Moved, TreeItemPtr.ToSharedRef());
			}
		}
		UE_LOGF(LogSceneOutliner, VeryVerbose, "Refresh requested by FSceneOutlinerHierarchyChangedData::Moved");

		Refresh();
	}
	else if (Event.Type == FSceneOutlinerHierarchyChangedData::FolderMoved)
	{
		check (Event.ItemIDs.Num() == Event.NewPaths.Num());
		for (int32 i = 0; i < Event.ItemIDs.Num(); ++i)
		{
			FSceneOutlinerTreeItemPtr Item = TreeItemMap.FindRef(Event.ItemIDs[i]);
			if (Item.IsValid())
			{
				// Remove it from the map under the old ID (which is derived from the folder path)
				TreeItemMap.Remove(Item->GetID());

				// Now change the path and put it back in the map with its new ID
				auto Folder = StaticCastSharedPtr<FFolderTreeItem>(Item);
				check(Event.NewPaths[i].GetRootObject() == Folder->GetRootObject());
				Folder->SetPath(Event.NewPaths[i].GetPath());

				TreeItemMap.Add(Item->GetID(), Item);

				// Add an operation to move the item in the hierarchy
				PendingOperations.Emplace(SceneOutliner::FPendingTreeOperation::Moved, Item.ToSharedRef());
			}
		}
		
		UE_LOGF(LogSceneOutliner, VeryVerbose, "Refresh requested by FSceneOutlinerHierarchyChangedData::FolderMoved");
		Refresh();
	}
	else if (Event.Type == FSceneOutlinerHierarchyChangedData::FullRefresh)
	{
		UE_LOGF(LogSceneOutliner, VeryVerbose, "Full Refresh requested by FSceneOutlinerHierarchyChangedData::FullRefresh");
		FullRefresh();
	}
}

void SSceneOutliner::PostUndo(bool bSuccess)
{
	// Refresh our tree in case any changes have been made to the scene that might effect our list
	if( !bIsReentrant )
	{
		UE_LOGF(LogSceneOutliner, VeryVerbose, "FullRefresh requested by SSceneOutliner::PostUndo");
		FullRefresh();
	}
}

void SSceneOutliner::OnItemLabelChanged(FSceneOutlinerTreeItemPtr ChangedItem, bool bFlashHighlight)
{
	// If the item already exists
	if (FSceneOutlinerTreeItemPtr* ExistingItem = TreeItemMap.Find(ChangedItem->GetID()))
	{
		(*ExistingItem)->OnLabelChanged();
		
		// The changed item flags will have been set already
		if (!ChangedItem->Flags.bIsFilteredOut)
		{
			if (bFlashHighlight)
			{
				OutlinerTreeView->FlashHighlightOnItem(*ExistingItem);
			}
			RequestSort();
		}
		else
		{
			// No longer matches the filters, remove it
			PendingOperations.Emplace(SceneOutliner::FPendingTreeOperation::Removed, ExistingItem->ToSharedRef());
			PendingTreeItemMap_Removal.Add(ChangedItem->GetID(), ExistingItem->ToSharedRef());
			Refresh();
		}
	}
	else if (FSceneOutlinerTreeItemPtr* PendingItem = PendingTreeItemMap.Find(ChangedItem->GetID()))
	{
		(*PendingItem)->OnLabelChanged();
	}
	else
	{
		// Attempt to add the item if we didn't find it - perhaps it now matches the filter?
		if (ChangedItem && !ChangedItem->Flags.bIsFilteredOut)
		{
			AddPendingItemAndChildren(ChangedItem);
		}
	}
}

void SSceneOutliner::BindCommands(const TSharedPtr<FUICommandList>& InCommandListToAppend)
{
	CommandList = MakeShared<FUICommandList>();

	// Bind Mode Commands
	if (Mode)
	{
		Mode->BindCommands(CommandList.ToSharedRef());
	}

	// Append additional commands
	if (InCommandListToAppend.IsValid())
	{
		CommandList->Append(InCommandListToAppend.ToSharedRef());
	}
}

void SSceneOutliner::OnAssetReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	if (InPackageReloadPhase == EPackageReloadPhase::PostBatchPostGC)
	{
		// perhaps overkill but a simple Refresh() doesn't appear to work.
		UE_LOGF(LogSceneOutliner, VeryVerbose, "FullRefresh requested by SSceneOutliner::OnAssetReloaded");
		FullRefresh();
	}
}

void SSceneOutliner::OnFilterTextChanged( const FText& InFilterText )
{
	SearchBoxFilter->SetRawFilterText( InFilterText );
	FilterTextBoxWidget->SetError( SearchBoxFilter->GetFilterErrorText() );

	Mode->OnFilterTextChanged(InFilterText);
}

void SSceneOutliner::OnFilterTextCommitted( const FText& InFilterText, ETextCommit::Type CommitInfo )
{
	const FString CurrentFilterText = InFilterText.ToString();
	// We'll only select items if the user actually pressed the enter key. We don't want to change
	// selection just because focus was lost from the search text field.
	if( CommitInfo == ETextCommit::OnEnter )
	{
		// Any text in the filter?  If not, we won't bother doing anything
		if( !CurrentFilterText.IsEmpty() )
		{
			FSceneOutlinerItemSelection Selection;

			// Gather all of the items that match the filter text
			for (auto& Pair : TreeItemMap)
			{
				Pair.Value->Flags.bIsFilteredOut = !PassesAllFilters(Pair.Value);
				if (!Pair.Value->Flags.bIsFilteredOut)
				{
					Selection.Add(Pair.Value);
				}
			}

			Mode->OnFilterTextCommited(Selection, CommitInfo);
		}
	}
	else if (CommitInfo == ETextCommit::OnCleared)
	{
		OnFilterTextChanged(InFilterText);
	}
}

EVisibility SSceneOutliner::GetFilterStatusVisibility() const
{
	return IsTextFilterActive() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SSceneOutliner::GetEmptyLabelVisibility() const
{
	return ( IsTextFilterActive() || RootTreeItems.Num() > 0 ) ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SSceneOutliner::GetFilterStatusText() const
{
	return Mode->GetStatusText();
}

FSlateColor SSceneOutliner::GetFilterStatusTextColor() const
{
	return Mode->GetStatusTextColor();
}

bool SSceneOutliner::IsTextFilterActive() const
{
	return FilterTextBoxWidget->GetText().ToString().Len() > 0;
}

const FSlateBrush* SSceneOutliner::GetFilterButtonGlyph() const
{
	if( IsTextFilterActive() )
	{
		return FAppStyle::GetBrush(TEXT("SceneOutliner.FilterCancel"));
	}
	else
	{
		return FAppStyle::GetBrush(TEXT("SceneOutliner.FilterSearch"));
	}
}

FString SSceneOutliner::GetFilterButtonToolTip() const
{
	return IsTextFilterActive() ? LOCTEXT("ClearSearchFilter", "Clear search filter").ToString() : LOCTEXT("StartSearching", "Search").ToString();

}

TAttribute<FText> SSceneOutliner::GetFilterHighlightText() const
{
	return TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic([](TWeakPtr<SceneOutliner::TreeItemTextFilter> Filter){
		auto FilterPtr = Filter.Pin();
		return FilterPtr.IsValid() ? FilterPtr->GetRawFilterText() : FText();
	}, TWeakPtr<SceneOutliner::TreeItemTextFilter>(SearchBoxFilter)));
}

SceneOutliner::TreeItemTextFilter::FChangedEvent& SSceneOutliner::OnTextFilterChanged() const
{
	check(SearchBoxFilter.IsValid());
	return SearchBoxFilter->OnChanged();
}

void SSceneOutliner::SetKeyboardFocus()
{
	if (SupportsKeyboardFocus())
	{
		FWidgetPath OutlinerTreeViewWidgetPath;
		// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
		FSlateApplication::Get().GeneratePathToWidgetUnchecked( OutlinerTreeView.ToSharedRef(), OutlinerTreeViewWidgetPath );
		FSlateApplication::Get().SetKeyboardFocus( OutlinerTreeViewWidgetPath, EFocusCause::SetDirectly );
	}
}

const FSlateBrush* SSceneOutliner::GetCachedIconForClass(FName InClassName) const
{ 
	if (CachedIcons.Find(InClassName))
	{
		return *CachedIcons.Find(InClassName);
	}
	else
	{
		return nullptr;
	}
}

void SSceneOutliner::CacheIconForClass(FName InClassName, const FSlateBrush* InSlateBrush)
{
	CachedIcons.Emplace(InClassName, InSlateBrush);
}

bool SSceneOutliner::SupportsKeyboardFocus() const
{
	return Mode->SupportsKeyboardFocus();
}

FReply SSceneOutliner::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (CommandList.IsValid())
	{
		if (CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	}

	// Fallback to the Mode OnKeyDown to check if it's handled there
	return Mode->OnKeyDown(InKeyEvent);
}

void SSceneOutliner::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{	
	TRACE_CPUPROFILER_EVENT_SCOPE(SSceneOutliner::Tick);

	Mode->Tick();

	for (auto& Pair : Columns)
	{
		Pair.Value->Tick(InCurrentTime, InDeltaTime);
	}

	if ( bPendingFocusNextFrame && FilterTextBoxWidget->GetVisibility() == EVisibility::Visible )
	{
		FWidgetPath WidgetToFocusPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked( FilterTextBoxWidget.ToSharedRef(), WidgetToFocusPath );
		FSlateApplication::Get().SetKeyboardFocus( WidgetToFocusPath, EFocusCause::SetDirectly );
		bPendingFocusNextFrame = false;
	}

	if ( bNeedsColumRefresh )
	{
		SetupColumns();
	}

	if( bNeedsRefresh )
	{
		if( !bIsReentrant )
		{
			if (Mode->CanPopulate())
			{
				Populate();
			}
		}
	}

	// If we are pending a sort
	if (bSortDirty)
	{
		bool bExecuteSort = false;
		
		SortOutlinerTimer -= InDeltaTime;
		UIRefreshDelay -= InDeltaTime;

		// If we are in PIE, we will use SortOutlinerTimer as the min delay between sorts
		if (GEditor->PlayWorld)
		{
			if (SortOutlinerTimer <= 0)
			{
				bExecuteSort = true;
			}
		}
		// In editor worlds, we will check if there is any user specific UIRefreshDelay
		else
		{
			if (UIRefreshDelay <= 0)
			{
				bExecuteSort = true;
			}
		}

		// If the column is doing any work to prepare the sort, make sure it is ready before actually sorting the items
		if (TSharedPtr<ISceneOutlinerColumn> Column = Columns.FindRef(SortByColumn))
		{
			bExecuteSort &= Column->IsSortReady();
		}

		if (bExecuteSort)
		{
			UE_LOGF(LogSceneOutliner, VeryVerbose, "Sort Executed");

			SortItems(RootTreeItems);

			// Also update expansion state based on item states
			// This is done here because updating expansion state causes a refresh, so we want to make sure that does not ignore any UIRefreshDelay
			bForceParentItemsExpanded = !SearchBoxFilter->GetRawFilterText().IsEmpty();
			
			for (const auto& Pair : TreeItemMap)
			{
				Pair.Value->Flags.bChildrenRequireSort = true;
				if (Pair.Value->GetChildren().Num())
				{
					OutlinerTreeView->SetItemExpansion(Pair.Value, bForceParentItemsExpanded || Pair.Value->Flags.bIsExpanded);
				}
			}
			
			OutlinerTreeView->RequestTreeRefresh();
			bSortDirty = false;

			// Reset both timers
			SortOutlinerTimer = SCENE_OUTLINER_RESORT_TIMER;
			UIRefreshDelay = 0.0f;
		}
	}
	
	if (bSelectionDirty)
	{
		Mode->SynchronizeSelection();
		bSelectionDirty = false;
	}
}

EColumnSortMode::Type SSceneOutliner::GetColumnSortMode( const FName ColumnId ) const
{
	if (SortByColumn == ColumnId)
	{
		auto Column = Columns.FindRef(ColumnId);
		if (Column.IsValid() && Column->SupportsSorting())
		{
			return SortMode;
		}
	}

	return EColumnSortMode::None;
}

void SSceneOutliner::OnColumnSortModeChanged( const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode )
{
	auto Column = Columns.FindRef(ColumnId);
	if (!Column.IsValid() || !Column->SupportsSorting())
	{
		return;
	}

	SortByColumn = ColumnId;
	SortMode = InSortMode;

	RequestSort();
}

void SSceneOutliner::RequestSort()
{
	bSortDirty = true;

	if (TSharedPtr<ISceneOutlinerColumn> Column = Columns.FindRef(SortByColumn))
	{
		Column->OnSortRequested(EColumnSortPriority::Primary, SortMode);
	}
}

void SSceneOutliner::SortItems(TArray<FSceneOutlinerTreeItemPtr>& Items) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSceneOutliner::SortItems);

	auto Column = Columns.FindRef(SortByColumn);
	if (Column.IsValid())
	{
		Column->SortItems(Items, SortMode);
	}
}

uint32 SSceneOutliner::GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const 
{
	return Mode->GetTypeSortPriority(Item);
}

void SSceneOutliner::ExpandAll()
{
	for (FSceneOutlinerTreeItemPtr& Item : RootTreeItems)
	{
		SetItemExpansionRecursive(Item, true);
	}
}

void SSceneOutliner::CollapseAll()
{
	for (FSceneOutlinerTreeItemPtr& Item : RootTreeItems)
	{
		SetItemExpansionRecursive(Item, false);
	}
}

void SSceneOutliner::SetItemExpansionRecursive(FSceneOutlinerTreeItemPtr Model, bool bInExpansionState)
{
	if (Model.IsValid())
	{
		OutlinerTreeView->SetItemExpansion(Model, bInExpansionState);
		for (auto& Child : Model->Children)
		{
			if (Child.IsValid())
			{
				SetItemExpansionRecursive(Child.Pin(), bInExpansionState);
			}
		}
	}
}

TSharedPtr<FDragDropOperation> SSceneOutliner::CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const
{ 
	return Mode->CreateDragDropOperation(MouseEvent, InTreeItems); 
}

/** Parse a drag drop operation into a payload */
bool SSceneOutliner::ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const 
{
	return Mode->ParseDragDrop(OutPayload, Operation);
}

/** Validate a drag drop operation on a drop target */
FSceneOutlinerDragValidationInfo SSceneOutliner::ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const 
{ 
	return Mode->ValidateDrop(DropTarget, Payload); 
}

/** Called when a payload is dropped onto a target */
void SSceneOutliner::OnDropPayload(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const 
{ 
	return Mode->OnDrop(DropTarget, Payload, ValidationInfo); 
}

/** Called when a payload is dragged over an item */
FReply SSceneOutliner::OnDragOverItem(const FDragDropEvent& Event, const ISceneOutlinerTreeItem& Item) const 
{ 
	return Mode->OnDragOverItem(Event, Item); 
}

bool SSceneOutliner::UpdateOperationDecorator(const FDragDropEvent& Event, const FSceneOutlinerDragValidationInfo* ValidationInfo) const
{
	return Mode->UpdateOperationDecorator(Event, ValidationInfo); 
}

void SSceneOutliner::PinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems)
{
	Mode->PinItems(InItems);
}

void SSceneOutliner::UnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems)
{
	Mode->UnpinItems(InItems);
}

bool SSceneOutliner::CanPinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const
{
	return Mode->CanPinItems(InItems);
}

bool SSceneOutliner::CanUnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const
{
	return Mode->CanUnpinItems(InItems);
}

void SSceneOutliner::PinSelectedItems()
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems;
	GetSelection().Get(SelectedItems);
	PinItems(SelectedItems);
}

void SSceneOutliner::UnpinSelectedItems()
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems;
	GetSelection().Get(SelectedItems);
	UnpinItems(SelectedItems);
}

bool SSceneOutliner::CanPinSelectedItems() const
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems;
	GetSelection().Get(SelectedItems);
	return CanPinItems(SelectedItems);
}

bool SSceneOutliner::CanUnpinSelectedItems() const
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems;
	GetSelection().Get(SelectedItems);
	return CanUnpinItems(SelectedItems);
}

void SSceneOutliner::FrameSelectedItems()
{
	TArray<TSharedPtr<ISceneOutlinerTreeItem>> SelectedItems = GetSelectedItems();
	if (!SelectedItems.IsEmpty())
	{
		ScrollItemIntoView(SelectedItems.Last());
	}
}

void SSceneOutliner::FrameItem(const FSceneOutlinerTreeItemID& Item)
{
	if (const TSharedPtr<ISceneOutlinerTreeItem>* TreeItem = TreeItemMap.Find(Item))
	{
		ScrollItemIntoView(*TreeItem);
	}
}

FSceneOutlinerTreeItemPtr SSceneOutliner::FindParent(const ISceneOutlinerTreeItem& InItem) const
{
	FSceneOutlinerTreeItemPtr Parent = Mode->GetHierarchy()->FindOrCreateParentItem(InItem, TreeItemMap, /*bCreate=*/false);
	if (!Parent.IsValid())
	{
		Parent = Mode->GetHierarchy()->FindOrCreateParentItem(InItem, PendingTreeItemMap, /*bCreate=*/false);
	}
	return Parent;
}

void SSceneOutliner::ToggleStackHierarchyHeaders()
{
	bShouldStackHierarchyHeaders = !bShouldStackHierarchyHeaders;

	FSceneOutlinerConfig* OutlinerConfig = GetMutableConfig();

	if (OutlinerConfig != nullptr)
	{
		OutlinerConfig->bShouldStackHierarchyHeaders = bShouldStackHierarchyHeaders;
		SaveConfig();
	}

	FullRefresh();
}

bool SSceneOutliner::ShouldStackHierarchyHeaders() const
{
	return bShouldStackHierarchyHeaders;
}

struct FSceneOutlinerConfig* SSceneOutliner::GetMutableConfig()
{
	if (OutlinerIdentifier.IsNone())
	{
		return nullptr;
	}

	return &UOutlinerConfig::Get()->Outliners.FindOrAdd(OutlinerIdentifier);
}


const FSceneOutlinerConfig* SSceneOutliner::GetConstConfig() const
{
	if (OutlinerIdentifier.IsNone())
	{
		return nullptr;
	}

	return UOutlinerConfig::Get()->Outliners.Find(OutlinerIdentifier);
}

void SSceneOutliner::SaveConfig()
{
	UOutlinerConfig::Get()->SaveEditorConfig();
}

void FSceneOutlinerMenuHelper::AddMenuEntryCreateFolder(FToolMenuSection& InSection, SSceneOutliner& InOutliner)
{
	const FSlateIcon NewFolderIcon(FAppStyle::GetAppStyleSetName(), "SceneOutliner.NewFolderIcon");
	InSection.AddMenuEntry("CreateFolder", LOCTEXT("CreateFolder", "Create Folder"), LOCTEXT("CreateFolderTooltip", "Create Folder"), NewFolderIcon, FUIAction(FExecuteAction::CreateSP(&InOutliner, &SSceneOutliner::CreateFolder)));
}

void FSceneOutlinerMenuHelper::AddMenuEntryCleanupFolders(FToolMenuSection& InSection, ULevel* InLevel)
{
	if (InLevel && InLevel->IsUsingActorFolders())
	{
		const FSlateIcon CleanupFoldersIcon(FAppStyle::GetAppStyleSetName(), "SceneOutliner.CleanupActorFoldersIcon");
		InSection.AddMenuEntry("CleanupFolders", LOCTEXT("CleanupFolders", "Cleanup Folders"), LOCTEXT("CleanupFoldersTooltip", "Cleanup unreferenced and deleted actor folders"), CleanupFoldersIcon, FExecuteAction::CreateLambda([InLevel]()
		{
			const FScopedTransaction Transaction(LOCTEXT("CleanupFolders", "Cleanup Folders"));
			InLevel->CleanupDeletedAndUnreferencedActorFolders();
		}));
	}
}

TSharedPtr<FSceneOutlinerTreeItemSCC> SSceneOutliner::GetItemSourceControl(const FSceneOutlinerTreeItemPtr& InItem)
{
	return SourceControlHandler->GetItemSourceControl(InItem);
}

void SSceneOutliner::AddSourceControlMenuOptions(UToolMenu* Menu)
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems = GetTree().GetSelectedItems();
	SourceControlHandler->AddSourceControlMenuOptions(Menu, SelectedItems);
}

void SSceneOutliner::FilterByType(TSharedPtr<FCustomClassFilterData> CustomClassFilterData)
{
	if (FilterBar)
	{
		FilterBar->FilterByType(CustomClassFilterData);
	}
}

void SSceneOutliner::FilterByTypeCategory(TSharedPtr<FFilterCategory> TypeCategory, TArray<TSharedPtr<FCustomClassFilterData>> Classes)
{
	if (FilterBar)
	{
		FilterBar->FilterByTypeCategory(TypeCategory, Classes);
	}
}

bool SSceneOutliner::NeedsRefresh() const
{
	return bNeedsRefresh || bFullRefresh;
}

void SSceneOutliner::AddTypeFilterToFilterBar(const TSharedRef<FFilterBase<SceneOutliner::FilterBarType>>& InFilter,
	TArrayView<const TSharedPtr<FFilterCategory>> InAdditionalCategories) const
{
	if (FilterBar)
	{
		if (TSharedPtr<SSceneOutlinerFilterBar> OutlinerFilterBar = StaticCastSharedPtr<SSceneOutlinerFilterBar>(FilterBar))
		{
			OutlinerFilterBar->AddTypeFilter(InFilter, InAdditionalCategories);
		}
	}
}

void SSceneOutliner::CreateCustomTextFilter(const FCustomTextFilterData& InFilterData, bool bApplyFilter)
{
	if (FilterBar)
	{
		if (TSharedPtr<SSceneOutlinerFilterBar> OutlinerFilterBar = StaticCastSharedPtr<SSceneOutlinerFilterBar>(FilterBar))
		{
			OutlinerFilterBar->OnCreateCustomTextFilter(InFilterData, bApplyFilter);
		}
	}
}

void SSceneOutliner::ModifyCustomTextFilter(const FCustomTextFilterData& InFilterData, const TSharedPtr<ICustomTextFilter<SceneOutliner::FilterBarType>>& InFilter)
{
	if (FilterBar)
	{
		if (TSharedPtr<SSceneOutlinerFilterBar> OutlinerFilterBar = StaticCastSharedPtr<SSceneOutlinerFilterBar>(FilterBar))
		{
			OutlinerFilterBar->OnModifyCustomTextFilter(InFilterData, InFilter);
		}
	}
}

void SSceneOutliner::DeleteCustomTextFilter(const TSharedPtr<ICustomTextFilter<SceneOutliner::FilterBarType>>& InFilter)
{
	if (FilterBar)
	{
		if (TSharedPtr<SSceneOutlinerFilterBar> OutlinerFilterBar = StaticCastSharedPtr<SSceneOutlinerFilterBar>(FilterBar))
		{
			OutlinerFilterBar->OnDeleteCustomTextFilter(InFilter);
		}
	}
}

void SSceneOutliner::OnFilterBarFilterChanged()
{
	FilterCollection = FilterBar->GetAllActiveFilters();
	FilterBar->SaveSettings();
	
	FullRefresh();
}

bool SSceneOutliner::CompareItemWithClassName(SceneOutliner::FilterBarType InItem, const TSet<FTopLevelAssetPath>& AssetClassPaths) const
{
	return Mode->CompareItemWithClassName(InItem, AssetClassPaths);
}

void SSceneOutliner::CreateFilterBar(const FSceneOutlinerFilterBarOptions& FilterBarOptions)
{
	if (!FilterBarOptions.bHasFilterBar)
	{
		return;
	}

	FName FilterBarIdentifier = OutlinerIdentifier;

	SAssignNew(FilterBar, SSceneOutlinerFilterBar)
	.OnCompareItemWithClassNames(this, &SSceneOutliner::CompareItemWithClassName)
	.OnFilterChanged(this, &SSceneOutliner::OnFilterBarFilterChanged)
	.CustomClassFilters(FilterBarOptions.CustomClassFilters)
	.CustomFilters(FilterBarOptions.CustomFilters)
	.FilterSearchBox(FilterTextBoxWidget)
	.FilterBarIdentifier(FilterBarIdentifier)
	.UseSharedSettings(FilterBarOptions.bUseSharedSettings)
	.CategoryToExpand(FilterBarOptions.CategoryToExpand)
	.CreateTextFilter(SSceneOutlinerFilterBar::FCreateTextFilter::CreateLambda([this]()
	{
		TSharedPtr< SceneOutliner::TreeItemTextFilter > Filter = this->CreateTextFilter();

		return MakeShareable(new FCustomTextFilter<SceneOutliner::FilterBarType>(Filter));
	}));
}

bool SSceneOutliner::IsFilterActive(const FString& FilterName) const
{
	if (FilterBar.IsValid())
	{
		if (TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>> FoundFilter = FilterBar->GetFilter(FilterName))
		{
			return FilterBar->IsFilterActive(FoundFilter);
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE