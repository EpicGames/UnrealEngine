// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlWidget.h"
#include "UncontrolledChangelistsModule.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerSourceControlWidget"

void SSourceControlWidget::Construct(const FArguments& InArgs, TSharedPtr<FSceneOutlinerTreeItemSCC> InItemSourceControl)
{
	check(InItemSourceControl.IsValid());

	ItemSourceControl = InItemSourceControl;
	
	ItemSourceControl->OnSourceControlStateChanged.BindLambda([this, WeakThis = AsWeak()](FSourceControlStatePtr SourceControlState)
	{
		if (WeakThis.IsValid())
		{
			UpdateSourceControlState(SourceControlState);
		}
	});
	
	ItemSourceControl->OnUncontrolledChangelistsStateChanged.BindLambda([this, WeakThis = AsWeak()](TSharedPtr<FUncontrolledChangelistState> UncontrolledChangelistState)
	{
		if (WeakThis.IsValid())
		{
			UpdateUncontrolledChangelistState(UncontrolledChangelistState);
		}
	});
	
	SImage::Construct(
		SImage::FArguments()
		.ColorAndOpacity(this, &SSourceControlWidget::GetForegroundColor)
		.Image(FStyleDefaults::GetNoBrush()));

	UpdateSourceControlState(ItemSourceControl->GetSourceControlState());
}

FReply SSourceControlWidget::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	check(ItemSourceControl);

	FSourceControlStatePtr SourceControlState = ItemSourceControl->RefreshSourceControlState();
	UpdateSourceControlState(SourceControlState);
	return FReply::Handled();
}

void SSourceControlWidget::UpdateSourceControlState(FSourceControlStatePtr SourceControlState)
{
	TWeakPtr<FUncontrolledChangelistState> UncontrolledChangelistState = ItemSourceControl->GetUncontrolledChangelistState();
	
	UpdateWidget(SourceControlState, UncontrolledChangelistState.Pin());
}

void SSourceControlWidget::UpdateUncontrolledChangelistState(TSharedPtr<FUncontrolledChangelistState> UncontrolledChangelistState)
{
	UpdateWidget(ItemSourceControl->GetSourceControlState(), UncontrolledChangelistState);
}

void SSourceControlWidget::UpdateWidget(FSourceControlStatePtr SourceControlState, const TSharedPtr<FUncontrolledChangelistState>& UncontrolledChangelistState)
{
	RemoveAllLayers();

	static const FName UncontrolledChangelistIconName = TEXT("Icons.Unlink");

	// Custom logic to combine uncontrolled and source control state
	if (SourceControlState.IsValid() && UncontrolledChangelistState.IsValid())
	{
		// We use the uncontrolled icon as the base icon
		const FSlateBrush* ImageBrush = FAppStyle::GetBrush(UncontrolledChangelistIconName);

		// We use the overlay from the source control icon (if any)
		const FSlateBrush* OverlayBrush = SourceControlState->GetIcon().GetOverlayIcon();

		// We use the color of the source control icon
		FSlateColor IconColor = SourceControlState->GetIcon().GetIcon()->TintColor;

		SetImage(ImageBrush);

		if (OverlayBrush)
		{
			AddLayer(OverlayBrush);
		}
		
		SetColorAndOpacity(IconColor);

		SetToolTipText(FText::Format(LOCTEXT("UncontrolledSourceControlCombinedTooltip",
			"{0}\nStatus in revision control: {1}"),
			UncontrolledChangelistState->GetDisplayTooltip(), SourceControlState->GetDisplayTooltip()));
	}
	else if (SourceControlState.IsValid() && !UncontrolledChangelistState.IsValid())
	{
		SetFromSlateIcon(SourceControlState->GetIcon());
		SetToolTipText(SourceControlState->GetDisplayTooltip());
	}
	else if (!SourceControlState.IsValid() && UncontrolledChangelistState.IsValid())
	{
		SetImage(FAppStyle::GetBrush(UncontrolledChangelistIconName));
		SetToolTipText(UncontrolledChangelistState->GetDisplayTooltip());
	}
	else
	{
		SetImage(nullptr);
		SetToolTipText(TAttribute<FText>());
	}
}

#undef LOCTEXT_NAMESPACE