// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Images/SLayeredImage.h"
#include "SourceControlHelpers.h"
#include "SceneOutlinerTreeItemSCC.h"

template<>
struct TWidgetTypeTraits<class SSourceControlWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SSourceControlWidget : public SLayeredImage
{
public:
	SLATE_BEGIN_ARGS(SSourceControlWidget) {}
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, TSharedPtr<FSceneOutlinerTreeItemSCC> InItemSourceControl);

private:

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	void UpdateSourceControlState(FSourceControlStatePtr SourceControlState);
	void UpdateUncontrolledChangelistState(TSharedPtr<FUncontrolledChangelistState> UncontrolledChangelistState);

	void UpdateWidget(FSourceControlStatePtr SourceControlState, const TSharedPtr<FUncontrolledChangelistState>& UncontrolledChangelistState);

	/** The object that keeps the source control state for the outliner */
	TSharedPtr<FSceneOutlinerTreeItemSCC> ItemSourceControl;
};
