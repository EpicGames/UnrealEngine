// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerActorInfoColumn.h"
#include "Engine/Blueprint.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Misc/StringBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "EditorClassUtils.h"
#include "SortHelper.h"
#include "ISceneOutliner.h"
#include "ActorTreeItem.h"
#include "ActorDescTreeItem.h"
#include "ComponentTreeItem.h"
#include "FolderTreeItem.h"
#include "LevelTreeItem.h"
#include "SceneOutlinerHelpers.h"
#include "WorldTreeItem.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerActorInfoColumn"

struct FGetInfo
{

	FString operator()(const ISceneOutlinerTreeItem& Item) const
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			AActor* Actor = ActorItem->Actor.Get();
			if (!Actor)
			{
				return FString();
			}

			return Actor->GetClass()->GetName();
		}
		else if (Item.IsA<FFolderTreeItem>())
		{
			return LOCTEXT("FolderTypeName", "Folder").ToString();
		}
		else if (Item.IsA<FWorldTreeItem>())
		{
			return LOCTEXT("WorldTypeName", "World").ToString();
		}
		else if (Item.IsA<FLevelTreeItem>())
		{
			return LOCTEXT("LevelTypeName", "Level").ToString();
		}
		else if (const FComponentTreeItem* ComponentItem = Item.CastTo<FComponentTreeItem>())
		{
			return LOCTEXT("ComponentTypeName", "Component").ToString();
		}
		else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				return ActorDescInstance->GetDisplayClassNameString();
			}
		}

		return FString();
	}
};

FTypeInfoColumn::FTypeInfoColumn( ISceneOutliner& Outliner)
	: SceneOutlinerWeak( StaticCastSharedRef<ISceneOutliner>(Outliner.AsShared()) )
{

}


FName FTypeInfoColumn::GetColumnID()
{
	return GetID();
}


SHeaderRow::FColumn::FArguments FTypeInfoColumn::ConstructHeaderRowColumn()
{
	/** Customizable actor data column */
	return SHeaderRow::Column( GetColumnID() )
		.FillWidth(2)
		.HeaderComboVisibility(EHeaderComboVisibility::OnHover)
		.DefaultTooltip(MakeComboToolTipText())
		.HeaderContent()
		[
			SNew( SHorizontalBox )

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text( this, &FTypeInfoColumn::GetSelectedMode )
			]
		];
}

const TSharedRef< SWidget > FTypeInfoColumn::ConstructRowWidget( FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row )
{
	auto SceneOutliner = SceneOutlinerWeak.Pin();
	check(SceneOutliner.IsValid());

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	TSharedRef<STextBlock> MainText = SNew( STextBlock )
		.Text( this, &FTypeInfoColumn::GetTextForItem, TWeakPtr<ISceneOutlinerTreeItem>(TreeItem) )
		.HighlightText( SceneOutliner->GetFilterHighlightText() )
		.ColorAndOpacity( FSlateColor::UseSubduedForeground() );

	HorizontalBox->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(8, 0, 0, 0)
	[
		MainText
	];

	TSharedPtr<SWidget> Hyperlink = ConstructClassHyperlink(*TreeItem);
	if (Hyperlink.IsValid())
	{
		// If we got a hyperlink, disable hide default text, and show the hyperlink
		MainText->SetVisibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FTypeInfoColumn::GetColumnDataVisibility, false)));
		Hyperlink->SetVisibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FTypeInfoColumn::GetColumnDataVisibility, true)));

		HorizontalBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(8, 0, 0, 0)
		[
			// Make sure that the hyperlink shows as black (by multiplying black * desired color) when selected so it is readable against the orange background even if blue/green/etc... normally
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.ForegroundColor_Static([](TWeakPtr<const STableRow<FSceneOutlinerTreeItemPtr>> WeakRow)->FSlateColor{
				auto TableRow = WeakRow.Pin();
				return TableRow.IsValid() && TableRow->IsSelected() ? FStyleColors::ForegroundHover : FSlateColor::UseStyle();
			}, TWeakPtr<const STableRow<FSceneOutlinerTreeItemPtr>>(StaticCastSharedRef<const STableRow<FSceneOutlinerTreeItemPtr>>(Row.AsShared())))
			[
				Hyperlink.ToSharedRef()
			]
		];
	}

	return HorizontalBox;
}

TSharedPtr<SWidget> FTypeInfoColumn::ConstructClassHyperlink( ISceneOutlinerTreeItem& TreeItem )
{
	if (const FActorTreeItem* ActorItem = TreeItem.CastTo<FActorTreeItem>())
	{
		if (AActor* Actor = ActorItem->Actor.Get())
		{
			return SceneOutliner::FSceneOutlinerHelpers::GetClassHyperlink(Actor);
		}
	}

	return nullptr;
}

void FTypeInfoColumn::PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const
{
	{
		FString String = FGetInfo()(Item);
		if (String.Len())
		{
			OutSearchStrings.Add(MoveTemp(String));
		}
	}
}

bool FTypeInfoColumn::SupportsSorting() const
{
	return true;
}

void FTypeInfoColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const
{
	FSceneOutlinerSortHelper<FString>()
		.Primary(FGetInfo(), SortMode)
		.Sort(RootItems);
}

EVisibility FTypeInfoColumn::GetColumnDataVisibility( bool bIsClassHyperlink ) const
{
	return bIsClassHyperlink ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FTypeInfoColumn::GetTextForItem( TWeakPtr<ISceneOutlinerTreeItem> TreeItem ) const
{
	auto Item = TreeItem.Pin();
	return Item.IsValid() ? FText::FromString(FGetInfo()(*Item)) : FText::GetEmpty();
}

FText FTypeInfoColumn::GetSelectedMode() const
{
	return MakeComboText();
}

FText FTypeInfoColumn::MakeComboText() const
{
	FText ModeName = LOCTEXT("CustomColumnMode_Class", "Type");

	return ModeName;
}


FText FTypeInfoColumn::MakeComboToolTipText() const
{
	FText ToolTipText = LOCTEXT("CustomColumnModeToolTip_Class", "Displays the name of each actor's type");
	return ToolTipText;
}

#undef LOCTEXT_NAMESPACE
