// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "SceneOutlinerPublicTypes.h"
#include "Misc/EnumRange.h"
#include "ISceneOutlinerColumn.h"

class ISceneOutliner;

/**
 * A custom column for the SceneOutliner which is used to display the Class/Type of Actors
   Note: Used to have support for displaying a variety of details related to Actors, that functionality
   has been moved FTextInfoColumn (See SceneOutlinerModule::CreateActorInfoColumns)
 */
class FTypeInfoColumn : public ISceneOutlinerColumn
{

public:

	/**
	 *	Constructor
	 */
	FTypeInfoColumn( ISceneOutliner& Outliner);

	virtual ~FTypeInfoColumn() = default;

	static FName GetID() { return FSceneOutlinerBuiltInColumnTypes::ActorInfo(); }

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation

	virtual FName GetColumnID() override;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;

	virtual const TSharedRef< SWidget > ConstructRowWidget( FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row ) override;

	virtual void PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const override;

	virtual bool SupportsSorting() const override;

	virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const override;
	
	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////

	FText GetTextForItem( const TWeakPtr<ISceneOutlinerTreeItem> TreeItem ) const;

private:

	TSharedPtr<SWidget> ConstructClassHyperlink( ISceneOutlinerTreeItem& TreeItem );

	EVisibility GetColumnDataVisibility( bool bIsClassHyperlink ) const;

	FText MakeComboText( ) const;

	FText MakeComboToolTipText( ) const;

	FText GetSelectedMode() const;

private:

	/** Weak reference to the outliner widget that owns our list */
	TWeakPtr< ISceneOutliner > SceneOutlinerWeak;
};
