// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SceneOutlinerConfig)

TObjectPtr<UOutlinerConfig> UOutlinerConfig::Instance = nullptr;

void UOutlinerConfig::Initialize()
{
	if(!Instance)
	{
		Instance = NewObject<UOutlinerConfig>(); 
		Instance->AddToRoot();
	}
}
