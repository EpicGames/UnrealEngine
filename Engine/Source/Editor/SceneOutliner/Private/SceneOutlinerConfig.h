// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "EditorConfigBase.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "SceneOutlinerConfig.generated.h"

class FName;
class UObject;


USTRUCT()
struct FSceneOutlinerConfig
{
	GENERATED_BODY()

public:
	
	/** Map to store the visibility of each column */
	UPROPERTY()
	TMap<FName, bool> ColumnVisibilities;

	/** Whether the hierarchy is pinned at the top of the outliner*/
	UPROPERTY()
	bool bShouldStackHierarchyHeaders = true;
};


UCLASS(EditorConfig="Outliner")
class UOutlinerConfig : public UEditorConfigBase
{
	GENERATED_BODY()
	
public:

	static void Initialize();
	static UOutlinerConfig* Get() { return Instance; }

	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FSceneOutlinerConfig> Outliners;

private:

	static TObjectPtr<UOutlinerConfig> Instance;
};