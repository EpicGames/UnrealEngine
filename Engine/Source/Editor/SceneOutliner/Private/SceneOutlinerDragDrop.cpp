// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerDragDrop.h"

#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSceneOutliner"

FSceneOutlinerDragDropOp::FSceneOutlinerDragDropOp()
	: OverrideText()
	, OverrideIcon(nullptr)
{
		
}
	
EVisibility FSceneOutlinerDragDropOp::GetOverrideVisibility() const
{
	return OverrideText.IsEmpty() && OverrideIcon == nullptr ? EVisibility::Collapsed : EVisibility::Visible;
}
	
EVisibility FSceneOutlinerDragDropOp::GetDefaultVisibility() const
{
	return OverrideText.IsEmpty() && OverrideIcon == nullptr ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedPtr<SWidget> FSceneOutlinerDragDropOp::GetDefaultDecorator() const
{
	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	VerticalBox->AddSlot()
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
		.Visibility(this, &FSceneOutlinerDragDropOp::GetOverrideVisibility)
		.Content()
		[			
			SNew(SHorizontalBox)
				
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 0.0f, 0.0f, 3.0f, 0.0f )
			[
				SNew( SImage )
				.Image( this, &FSceneOutlinerDragDropOp::GetOverrideIcon )
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign( VAlign_Center )
			[
				SNew(STextBlock) 
				.Text( this, &FSceneOutlinerDragDropOp::GetOverrideText )
			]
		]
	];

	for (auto& SubOp : SubOps)
	{
		auto Content = SubOp->GetDefaultDecorator();
		if (Content.IsValid())
		{
			Content->SetVisibility(TAttribute<EVisibility>(this, &FSceneOutlinerDragDropOp::GetDefaultVisibility));
			VerticalBox->AddSlot()[Content.ToSharedRef()];
		}
	}

	return VerticalBox;
}

#undef LOCTEXT_NAMESPACE
