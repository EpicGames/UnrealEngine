// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerFilterBar.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerFilterBar"

const FName SSceneOutlinerFilterBar::SharedIdentifier("SceneOutlinerFilterBarSharedSettings");
SSceneOutlinerFilterBar::FCustomTextFilterEvent SSceneOutlinerFilterBar::CustomTextFilterEvent;

void SSceneOutlinerFilterBar::Construct( const FArguments& InArgs )
{
	bUseSharedSettings = InArgs._UseSharedSettings;
	this->CategoryToExpand = InArgs._CategoryToExpand;

	SFilterBar<SceneOutliner::FilterBarType>::FArguments Args;
	Args._OnFilterChanged = InArgs._OnFilterChanged;
	Args._CustomFilters = InArgs._CustomFilters;
	Args._UseDefaultAssetFilters = false;
	Args._CustomClassFilters = InArgs._CustomClassFilters;
	Args._CreateTextFilter = InArgs._CreateTextFilter;
	Args._FilterSearchBox = InArgs._FilterSearchBox;
	Args._FilterBarIdentifier = InArgs._FilterBarIdentifier;
	Args._OnCompareItemWithClassNames = InArgs._OnCompareItemWithClassNames;
	Args._OnConvertItemToAssetData = InArgs._OnConvertItemToAssetData;
	Args._FilterPillStyle = EFilterPillStyle::Basic;

	SFilterBar<SceneOutliner::FilterBarType>::Construct(Args);
	
	/* If we are using shared settings, add a default config for the shared settings in case it doesnt exist
	 * This needs to go after SAssetFilterBar<FAssetFilterType>::Construct() to ensure UFilterBarConfig is valid
	 */
	if(bUseSharedSettings)
	{
		UFilterBarConfig::Get()->FilterBars.FindOrAdd(SharedIdentifier);

		// Bind our delegate for when another SSceneOutlinerFilterBar creates a custom text filter, so we can sync our list
		CustomTextFilterEvent.AddSP(this, &SSceneOutlinerFilterBar::OnExternalCustomTextFilterCreated);
	}
}

void SSceneOutlinerFilterBar::SaveSettings()
{
	// If this instance doesn't want to use the shared settings, save the settings normally
	if(!bUseSharedSettings)
	{
		SAssetFilterBar<SceneOutliner::FilterBarType>::SaveSettings();
		return;
	}

	if(FilterBarIdentifier.IsNone())
	{
		UE_LOGF(LogSlate, Error, "SSceneOutlinerFilterBar Requires that you specify a FilterBarIdentifier to save settings");
		return;
	}

	// Get the settings unique to this instance and the common settings
	FFilterBarSettings* InstanceSettings = &UFilterBarConfig::Get()->FilterBars.FindOrAdd(FilterBarIdentifier);
	FFilterBarSettings* SharedSettings = &UFilterBarConfig::Get()->FilterBars.FindOrAdd(SharedIdentifier);

	// Empty both the configs, we are just going to re-save everything there
	InstanceSettings->Empty();
	SharedSettings->Empty();

	// Save all the programatically added filters normally
	SaveFilters(InstanceSettings);

	/** For each custom text filter: Save the filterdata into the common settings, so that all instances that use it
	 *	are synced.
	 *	For each CHECKED custom text filter: Save just the filter name, and the checked and active state into the
	 *	instance settings. Those are specific to this instance (i.e we don't want a filter to be active in all
	 *	instances if activated in one)
	 */
	for (const TSharedRef<ICustomTextFilter<SceneOutliner::FilterBarType>>& CustomTextFilter : this->CustomTextFilters)
	{
		// Get the actual FFilterBase
		TSharedRef<FFilterBase<SceneOutliner::FilterBarType>> CustomFilter = CustomTextFilter->GetFilter().ToSharedRef();
			
		// Is the filter "checked", i.e visible in the filter bar
		bool bIsChecked = IsFrontendFilterInUse(CustomFilter);

		// Is the filter "active", i.e visible and enabled in the filter bar
		bool bIsActive = IsFilterActive(CustomFilter);

		// Get the data associated with this filter
		FCustomTextFilterData FilterData = CustomTextFilter->CreateCustomTextFilterData();

		// Just save the filter data into the shared settings
		FCustomTextFilterState SharedFilterState;
		SharedFilterState.FilterData = FilterData;
		SharedSettings->CustomTextFilters.Add(SharedFilterState);

		if(bIsChecked)
		{
			// Create a duplicate filter data that just contains the filter label for this instance to know
			FCustomTextFilterData InstanceFilterData;
			InstanceFilterData.FilterLabel = FilterData.FilterLabel;
			
			// Just save the filter name and enabled/active state into the shared settings
			FCustomTextFilterState InstanceFilterState;
			InstanceFilterState.bIsChecked = bIsChecked;
			InstanceFilterState.bIsActive = bIsActive;
			InstanceFilterState.FilterData = InstanceFilterData;
			
			InstanceSettings->CustomTextFilters.Add(InstanceFilterState);
		}
	}

	SaveConfig();
}

void SSceneOutlinerFilterBar::LoadSettings()
{
	// If this instance doesn't want to use the shared settings, load the settings normally
	if(!bUseSharedSettings)
	{
		SAssetFilterBar<SceneOutliner::FilterBarType>::LoadSettings();
		return;
	}

	if(FilterBarIdentifier.IsNone())
	{
		UE_LOGF(LogSlate, Error, "SSceneOutlinerFilterBar Requires that you specify a FilterBarIdentifier to load settings");
		return;
	}

	// Get the settings unique to this instance and the common settings
	const FFilterBarSettings* InstanceSettings = UFilterBarConfig::Get()->FilterBars.Find(FilterBarIdentifier);
	const FFilterBarSettings* SharedSettings = UFilterBarConfig::Get()->FilterBars.Find(SharedIdentifier);

	// Load the filters specified programatically normally
	LoadFilters(InstanceSettings);

	// Load the custom text filters from the shared settings
	LoadCustomTextFilters(SharedSettings);
	
	// From the instance settings, get each checked filter and set the checked and active state
	for(const FCustomTextFilterState& FilterState : InstanceSettings->CustomTextFilters)
	{
		if(!RestoreCustomTextFilterState(FilterState))
		{
			UE_LOGF(LogSlate, Warning, "SSceneOutlinerFilterBar was unable to load the following custom text filter: %ls", *FilterState.FilterData.FilterLabel.ToString());
		}
	}

	this->OnFilterChanged.ExecuteIfBound();
}

void SSceneOutlinerFilterBar::AddTypeFilter(const TSharedRef<FFilterBase<SceneOutliner::FilterBarType>>& InFilter,
	TArrayView<const TSharedPtr<FFilterCategory>> InAdditionalCategories)
{
	TArray<TSharedPtr<FFilterCategory>>& Categories = TypeFilters.FindOrAdd(InFilter.ToSharedPtr());
	if (!InAdditionalCategories.IsEmpty())
	{
		for (const TSharedPtr<FFilterCategory>& Category : InAdditionalCategories)
		{
			Categories.AddUnique(Category);
		}
	}
	
	AddFilter(InFilter);
}

void SSceneOutlinerFilterBar::CreateTypeFiltersMenuSection(UToolMenu* InMenu,
	TArray<TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>> InFilters)
{
	FToolMenuSection& Section = InMenu->AddSection("Section");
	for (const TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>& Filter : InFilters)
	{
		if (!Filter.IsValid())
		{
			continue;
		}
		const TSharedRef<FFilterBase<SceneOutliner::FilterBarType>> FilterRef = Filter.ToSharedRef();
		Section.AddMenuEntry(
			NAME_None,
			Filter->GetDisplayName(),
			Filter->GetToolTipText(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), Filter->GetIconName()),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSceneOutlinerFilterBar::FrontendFilterClicked, FilterRef),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SSceneOutlinerFilterBar::IsFrontendFilterInUse, FilterRef)),
			EUserInterfaceActionType::ToggleButton);
	}
}

void SSceneOutlinerFilterBar::TypeFilterCategoryClicked(
	TArray<TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>> InFilters)
{
	const bool bFullCategoryInUse = IsTypeFilterCategoryChecked(InFilters) != ECheckBoxState::Unchecked;
	for (const TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>& Filter : InFilters)
	{
		if (!Filter.IsValid())
		{
			continue;
		}
		const TSharedRef<FFilterBase<SceneOutliner::FilterBarType>> FilterRef = Filter.ToSharedRef();
		if (bFullCategoryInUse || !IsFrontendFilterInUse(FilterRef))
		{
			FrontendFilterClicked(FilterRef);
		}
	}
}

ECheckBoxState SSceneOutlinerFilterBar::IsTypeFilterCategoryChecked(
	TArray<TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>> InFilters) const
{
	bool bIsAnyActionInUse = false, bIsAnyActionNotInUse = false;
	for (const TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>& Filter : InFilters)
	{
		if (!Filter.IsValid())
		{
			continue;
		}
		
		if (IsFrontendFilterInUse(Filter.ToSharedRef()))
		{
			bIsAnyActionInUse = true;
		}
		else
		{
			bIsAnyActionNotInUse = true;
		}

		if (bIsAnyActionInUse && bIsAnyActionNotInUse)
		{
			return ECheckBoxState::Undetermined;
		}
	}

	if (bIsAnyActionInUse)
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

void SSceneOutlinerFilterBar::PopulateAddFilterMenu(UToolMenu* Menu, TSharedPtr<FFilterCategory> MenuExpansion, FOnFilterAssetType OnFilterAssetType)
{
	using namespace UE::Editor::Widgets;

	// Build a map holding both FCustomClassFilterData entries and FFilterBase type
	// filters so the rest of the function only iterates one category rather than having them separate.
	TMap<TSharedPtr<FFilterCategory>, FTypeFilterCategoryEntry> TypeFilterMap;
	PopulateTypeFilterMap(OnFilterAssetType, TypeFilterMap);
	
	PopulateCommonFilterSections(Menu);

	auto BuildClassFilterDataEntry = [this](FToolMenuSection& Section, const TSharedPtr<FFilterCategory>& Category,
		const TArray<TSharedPtr<FCustomClassFilterData>>& Classes)
	{
		Section.AddSubMenu(
			FName(FText::AsCultureInvariant(Category->Title).ToString()),
			Category->Title,
			Category->Tooltip,
			FNewToolMenuDelegate::CreateSP(this, &SSceneOutlinerFilterBar::CreateFiltersMenuCategory, Classes),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSceneOutlinerFilterBar::FilterByTypeCategory, Category, Classes),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(this, &SSceneOutlinerFilterBar::IsTypeCategoryChecked, Category, Classes)),
			EUserInterfaceActionType::ToggleButton);
	};

	auto BuildBaseFilterEntry = [this](FToolMenuSection& Section, const TSharedPtr<FFilterCategory>& Category)
	{
		Section.AddSubMenu(
			FName(FText::AsCultureInvariant(Category->Title).ToString()),
			Category->Title,
			Category->Tooltip,
			FNewToolMenuDelegate::CreateSP(this, &SSceneOutlinerFilterBar::CreateOtherFiltersMenuCategory, Category),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSceneOutlinerFilterBar::FrontendFilterCategoryClicked, Category),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(this, &SSceneOutlinerFilterBar::IsFrontendFilterCategoryChecked, Category)),
			EUserInterfaceActionType::ToggleButton);
	};

	// Builds an entry for a FFilterBase (non-FCustomClassFilterData) Class Filter
	auto BuildBaseFilterClassEntry = [this](FToolMenuSection& Section, const TSharedPtr<FFilterCategory>& Category,
		const TArray<TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>>& InFilters)
	{
		Section.AddSubMenu(
			FName(FText::AsCultureInvariant(Category->Title).ToString()),
			Category->Title,
			Category->Tooltip,
			FNewToolMenuDelegate::CreateSP(this, &SSceneOutlinerFilterBar::CreateTypeFiltersMenuSection, InFilters),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSceneOutlinerFilterBar::TypeFilterCategoryClicked, InFilters),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(this, &SSceneOutlinerFilterBar::IsTypeFilterCategoryChecked, InFilters)),
			EUserInterfaceActionType::ToggleButton);
	};

	// "Other Filters" section: FFilterBase categories not present in TypeFilterMap
	TArray<TSharedPtr<FFilterCategory>> OtherCategories = AllFilterCategories;

	// Expand the designated expanded category, so it can appear before the Type Section
	if (MenuExpansion)
	{
		if (FTypeFilterCategoryEntry* ExpandedEntry = TypeFilterMap.Find(MenuExpansion))
		{
			FToolMenuSection& Section = Menu->AddSection(ExpandedEntry->SectionExtensionHook, ExpandedEntry->SectionHeading);
			FUIAction EntryAction;
			FNewToolMenuDelegate NewMenuDelegate;
			if (!ExpandedEntry->Classes.IsEmpty())
			{
				EntryAction = FUIAction(
					FExecuteAction::CreateSP(this, &SSceneOutlinerFilterBar::FilterByTypeCategory, MenuExpansion, ExpandedEntry->Classes),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SSceneOutlinerFilterBar::IsTypeCategoryChecked, MenuExpansion, ExpandedEntry->Classes));
				NewMenuDelegate = FNewToolMenuDelegate::CreateSP(this, &SSceneOutlinerFilterBar::CreateFiltersMenuCategory, ExpandedEntry->Classes);
			}
			else if (!ExpandedEntry->Filters.IsEmpty())
			{
				EntryAction = FUIAction(
					FExecuteAction::CreateSP(this, &SSceneOutlinerFilterBar::TypeFilterCategoryClicked, ExpandedEntry->Filters),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SSceneOutlinerFilterBar::IsTypeFilterCategoryChecked, ExpandedEntry->Filters));
				NewMenuDelegate = FNewToolMenuDelegate::CreateSP(this, &SSceneOutlinerFilterBar::CreateTypeFiltersMenuSection, ExpandedEntry->Filters);
			}
			else
			{
				EntryAction = FUIAction(
				FExecuteAction::CreateSP(this, &SSceneOutlinerFilterBar::FrontendFilterCategoryClicked, MenuExpansion),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SSceneOutlinerFilterBar::IsFrontendFilterCategoryChecked, MenuExpansion));
				NewMenuDelegate = FNewToolMenuDelegate::CreateSP(this, &SSceneOutlinerFilterBar::CreateOtherFiltersMenuCategory, MenuExpansion);
			}

			Section.AddMenuEntry(
				FName(FText::AsCultureInvariant(ExpandedEntry->SectionHeading).ToString()),
				ExpandedEntry->SectionHeading,
				MenuExpansion->Tooltip,
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PlacementBrowser.Icons.Basic"),
				EntryAction,
				EUserInterfaceActionType::ToggleButton);

			Section.AddSeparator("ExpandedCategorySeparator");

			NewMenuDelegate.Execute(Menu);

			TypeFilterMap.Remove(MenuExpansion);
			OtherCategories.Remove(MenuExpansion);
		}
	}
	
	// Sort all categories and their entries alphabetically
	TypeFilterMap.KeySort([](const TSharedPtr<FFilterCategory>& A, const TSharedPtr<FFilterCategory>& B) {
		return A->Title.CompareTo(B->Title) < 0;
	});
	
	for (TPair<TSharedPtr<FFilterCategory>, FTypeFilterCategoryEntry>& Pair : TypeFilterMap)
	{
		Pair.Value.Classes.Sort([](const TSharedPtr<FCustomClassFilterData>& A, const TSharedPtr<FCustomClassFilterData>& B) {
			return A->GetName().CompareTo(B->GetName()) < 0;
		});
	}

	FToolMenuSection& TypeSection = Menu->AddSection("AssetFilterBarFilterAdvancedAsset", LOCTEXT("TypeFiltersHeading", "Type Filters"));
	for (const TPair<TSharedPtr<FFilterCategory>, FTypeFilterCategoryEntry>& Pair : TypeFilterMap)
	{
		if (!Pair.Value.Classes.IsEmpty())
		{
			BuildClassFilterDataEntry(TypeSection, Pair.Key, Pair.Value.Classes);
		}
		else if (!Pair.Value.Filters.IsEmpty())
		{
			BuildBaseFilterClassEntry(TypeSection, Pair.Key, Pair.Value.Filters);
		}

		OtherCategories.Remove(Pair.Key);
	}

	if (!OtherCategories.IsEmpty())
	{
		FToolMenuSection& OtherSection = Menu->AddSection(
			"BasicFilterBarFiltersMenu",
			LOCTEXT("OtherFiltersHeading", "Other Filters"));

		for (const TSharedPtr<FFilterCategory>& Category : OtherCategories)
		{
			BuildBaseFilterEntry(OtherSection, Category);
		}
	}
}

void SSceneOutlinerFilterBar::PopulateTypeFilterMap(const FOnFilterAssetType& OnFilterAssetType, TMap<TSharedPtr<FFilterCategory>, FTypeFilterCategoryEntry>& TypeFilterMap)
{
	// Populate from FCustomClassFilterData via the SAssetFilterBar class helper
	TMap<TSharedPtr<FFilterCategory>, UE::Editor::Widgets::FFilterCategoryMenu> ClassMap =
		UE::Editor::Widgets::BuildCategoryToMenuMap(AssetFilterCategories, CustomClassFilters, OnFilterAssetType);
	for (TPair<TSharedPtr<FFilterCategory>, UE::Editor::Widgets::FFilterCategoryMenu>& Pair : ClassMap)
	{
		FTypeFilterCategoryEntry& Entry = TypeFilterMap.FindOrAdd(Pair.Key);
		Entry.SectionExtensionHook = Pair.Value.SectionExtensionHook;
		Entry.SectionHeading = Pair.Value.SectionHeading;
		Entry.Classes = MoveTemp(Pair.Value.Classes);
	}

	// Populate from FFilterBase type filters. Each filter is placed in its primary category and any additional categories
	// defined in AddTypeFilter.
	for (TPair<TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>, TArray<TSharedPtr<FFilterCategory>>>& TypeFilter : TypeFilters)
	{
		TArray<TSharedPtr<FFilterCategory>> AllCategories;
		if (TSharedPtr<FFilterCategory> PrimaryCategory = TypeFilter.Key->GetCategory())
		{
			AllCategories.Add(PrimaryCategory);
		}

		for (const TSharedPtr<FFilterCategory>& Category : TypeFilter.Value)
		{
			AllCategories.AddUnique(Category);
		}

		for (const TSharedPtr<FFilterCategory>& Category : AllCategories)
		{
			FTypeFilterCategoryEntry& Entry = TypeFilterMap.FindOrAdd(Category);
			if (Entry.SectionHeading.IsEmpty())
			{
				Entry.SectionHeading = FText::Format(LOCTEXT("ExpandedCategoryHeading", "{0} Filters"), Category->Title);
				Entry.SectionExtensionHook = FName(FText::AsCultureInvariant(Entry.SectionHeading).ToString());
			}
			Entry.Filters.AddUnique(TypeFilter.Key);
		}
	}
}

UAssetFilterBarContext* SSceneOutlinerFilterBar::CreateAssetFilterBarContext()
{
	UAssetFilterBarContext* AssetFilterBarContext = SFilterBar<const ISceneOutlinerTreeItem&>::CreateAssetFilterBarContext();
	AssetFilterBarContext->MenuExpansion = CategoryToExpand;

	// Use our PopulateAddFilterMenu whenever there are TEDS type filters to route into the
	// "Type Filters" section, or when a category needs to be expanded inline (the base class
	// PopulateCustomFilters has no per-category expansion support).
	if (!TypeFilters.IsEmpty() || CategoryToExpand.IsValid())
	{
		AssetFilterBarContext->PopulateFilterMenu = FOnPopulateAddAssetFilterMenu::CreateSP(this, &SSceneOutlinerFilterBar::PopulateAddFilterMenu);
	}

	return AssetFilterBarContext;
}

void SSceneOutlinerFilterBar::OnCreateCustomTextFilter(const FCustomTextFilterData& InFilterData, bool bApplyFilter)
{
	SAssetFilterBar<SceneOutliner::FilterBarType>::OnCreateCustomTextFilter(InFilterData, bApplyFilter);

	// If we are using shared settings (i.e sharing custom text filters) broadcast the event for all other instances to update
	if(bUseSharedSettings)
	{
		// First save the shared settings for other instances to use
		SaveSettings();
		CustomTextFilterEvent.Broadcast(AsShared());
	}
}

void SSceneOutlinerFilterBar::OnModifyCustomTextFilter(const FCustomTextFilterData& InFilterData, TSharedPtr<ICustomTextFilter<SceneOutliner::FilterBarType>> InFilter)
{
	SAssetFilterBar<SceneOutliner::FilterBarType>::OnModifyCustomTextFilter(InFilterData, InFilter);

	// If we are using shared settings (i.e sharing custom text filters) broadcast the event for all other instances to update
	if(bUseSharedSettings)
	{
		// First save the shared settings for other instances to use
		SaveSettings();
		CustomTextFilterEvent.Broadcast(AsShared());
	}
}

void SSceneOutlinerFilterBar::OnDeleteCustomTextFilter(const TSharedPtr<ICustomTextFilter<SceneOutliner::FilterBarType>> InFilter)
{
	SAssetFilterBar<SceneOutliner::FilterBarType>::OnDeleteCustomTextFilter(InFilter);

	// If we are using shared settings (i.e sharing custom text filters) broadcast the event for all other instances to update
	if(bUseSharedSettings)
	{
		// First save the shared settings for other instances to use
		SaveSettings();
		CustomTextFilterEvent.Broadcast(AsShared());
	}
}

void SSceneOutlinerFilterBar::LoadCustomTextFilters(const FFilterBarSettings* FilterBarConfig)
{
	CustomTextFilters.Empty();
	
	// Extract just the filter data from the common settings
	for(const FCustomTextFilterState& FilterState : FilterBarConfig->CustomTextFilters)
	{
		// Create an ICustomTextFilter using the provided delegate
		TSharedRef<ICustomTextFilter<SceneOutliner::FilterBarType>> NewTextFilter = this->CreateTextFilter.Execute().ToSharedRef();

		// Get the actual FFilterBase
		TSharedRef<FFilterBase<SceneOutliner::FilterBarType>> NewFilter = NewTextFilter->GetFilter().ToSharedRef();

		// Set the internals of the custom text filter from what we have saved
		NewTextFilter->SetFromCustomTextFilterData(FilterState.FilterData);

		// Add this to our list of custom text filters
		CustomTextFilters.Add(NewTextFilter);
	}
}


bool SSceneOutlinerFilterBar::RestoreCustomTextFilterState(const FCustomTextFilterState& InFilterState)
{
	// Find the filter associated with the current instance data from our list of custom text filters
	TSharedRef< ICustomTextFilter<SceneOutliner::FilterBarType> >* Filter =
		CustomTextFilters.FindByPredicate([&InFilterState](const TSharedRef< ICustomTextFilter<SceneOutliner::FilterBarType> >& Element)
	{
		return Element->CreateCustomTextFilterData().FilterLabel.EqualTo(InFilterState.FilterData.FilterLabel);
	});

	if(!Filter)
	{
		return false;
	}

	// Get the actual FFilterBase
	TSharedRef<FFilterBase<SceneOutliner::FilterBarType>> ActualFilter = Filter->Get().GetFilter().ToSharedRef();

	// Add it to the filter bar, since if it exists in this list it is checked
	TSharedRef<SFilter> AddedFilter = this->AddFilterToBar(ActualFilter);

	// Set the filter as active if it was previously
	AddedFilter->SetEnabled(InFilterState.bIsActive, false);
	this->SetFrontendFilterActive(ActualFilter, InFilterState.bIsActive);

	return true;
}

void SSceneOutlinerFilterBar::OnExternalCustomTextFilterCreated(TSharedPtr<SWidget> BroadcastingFilterBar)
{
	// Do nothing if we aren't using shared settings or if the event was broadcasted by this filter list
	if(!bUseSharedSettings || BroadcastingFilterBar == AsShared())
	{
		return;
	}

	/* We are going to remove all our custom text filters and re-load them from the shared settings, since a different
	 * instance modified them.
	 */

	// To preserve the state of any checked/active custom text filters
	TArray<FCustomTextFilterState> CurrentCustomTextFilterStates;
	
	for (const TSharedRef<ICustomTextFilter<SceneOutliner::FilterBarType>>& CustomTextFilter : this->CustomTextFilters)
	{
		// Get the actual FFilterBase
		TSharedRef<FFilterBase<SceneOutliner::FilterBarType>> CustomFilter = CustomTextFilter->GetFilter().ToSharedRef();
			
		// Is the filter "checked", i.e visible in the filter bar
		bool bIsChecked = IsFrontendFilterInUse(CustomFilter);

		// Is the filter "active", i.e visible and enabled in the filter bar
		bool bIsActive = IsFilterActive(CustomFilter);

		// Only save the state if the filter is checked so we can restore it
		if(bIsChecked)
		{
			/* Remove the filter from the list (calling SBasicFilterBar::RemoveFilter because we get a compiler error
			*  due to SAssetFilterBar overriding RemoveFilter that takes in an SFilter that hides the parent class function
			*/
			SBasicFilterBar<SceneOutliner::FilterBarType>::RemoveFilter(CustomFilter, false);
			
			FCustomTextFilterState FilterState;
			FilterState.FilterData = CustomTextFilter->CreateCustomTextFilterData();
			FilterState.bIsChecked = bIsChecked;
			FilterState.bIsActive = bIsActive;
			
			CurrentCustomTextFilterStates.Add(FilterState);
		}
	}

	// Get the shared settings and reload the filters
	FFilterBarSettings* SharedSettings = &UFilterBarConfig::Get()->FilterBars.FindOrAdd(SharedIdentifier);
	LoadCustomTextFilters(SharedSettings);

	// Restore the state of any previously active ones
	for(const FCustomTextFilterState& SavedFilterState : CurrentCustomTextFilterStates)
	{
		RestoreCustomTextFilterState(SavedFilterState);
	}

	// Save the settings for this instance
	SaveSettings();
}

#undef LOCTEXT_NAMESPACE