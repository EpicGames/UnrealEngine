// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SFilterBar.h"
#include "SceneOutlinerPublicTypes.h"

class SSceneOutlinerFilterBar : public SFilterBar< SceneOutliner::FilterBarType > 
{
public:
	
	/**
	 * An event delegate that is executed when a custom text filter has been created/modified/deleted in any SSceneOutlinerFilterBar
	 * that is using shared settings.
	 */
	DECLARE_EVENT_OneParam(SSceneOutlinerFilterBar, FCustomTextFilterEvent, TSharedPtr<SWidget> /* BroadcastingFilterBar */);
	
	using FOnFilterChanged = typename SBasicFilterBar<SceneOutliner::FilterBarType>::FOnFilterChanged;
	using FCompareItemWithClassNames = typename FAssetFilter<SceneOutliner::FilterBarType>::FCompareItemWithClassNames;
	using FCreateTextFilter = typename SBasicFilterBar<SceneOutliner::FilterBarType>::FCreateTextFilter;
	
	SLATE_BEGIN_ARGS( SSceneOutlinerFilterBar )
		: _FilterBarIdentifier(NAME_None)
		, _UseDefaultAssetFilters(true)
		{
		
		}

		/** Delegate for when filters have changed */
		SLATE_EVENT( FOnFilterChanged, OnFilterChanged )

		/** Specify this delegate to use asset comparison through IAssetRegistry, where you specify how to convert your Item into an FAssetData */
		SLATE_EVENT( FConvertItemToAssetData, OnConvertItemToAssetData )

		/** Specify this delegate to use asset comparison through IAssetRegistry, where you specify how to convert your Item into an FAssetData */
		SLATE_EVENT( FCompareItemWithClassNames, OnCompareItemWithClassNames )

		/** Initial List of Custom Filters that will be added to the AddFilter Menu */
		SLATE_ARGUMENT( TArray<TSharedRef<FFilterBase<SceneOutliner::FilterBarType>>>, CustomFilters)

		/** A unique identifier for this filter bar needed to enable saving settings in a config file */
		SLATE_ARGUMENT(FName, FilterBarIdentifier)
	
		/** Initial List of Custom Class Filters that will be added to the AddFilter Menu */
		SLATE_ARGUMENT( TArray<TSharedRef<FCustomClassFilterData>>, CustomClassFilters)
	
		/** Whether the filter bar should provide the default asset filters */
		SLATE_ARGUMENT(bool, UseDefaultAssetFilters)

		/** A delegate to create a TTextFilter for FilterType items. If provided, will allow creation of custom text filters
		 *  from the filter dropdown menu.
		 */
		SLATE_ARGUMENT(FCreateTextFilter, CreateTextFilter)
			
		/** An SFilterSearchBox that can be attached to this filter bar. When provided along with a CreateTextFilter
		 *  delegate, allows the user to save searches from the Search Box as text filters for the filter bar.
		 *	NOTE: Will bind a delegate to SFilterSearchBox::OnClickedAddSearchHistoryButton
		 */
		SLATE_ARGUMENT(TSharedPtr<SFilterSearchBox>, FilterSearchBox)

		/** If true, CustomTextFilters are saved/loaded from a config shared between multiple Filterbars
		 *	Currently used to sync all level editor outliner custom text filters.
		 */
		SLATE_ARGUMENT(bool, UseSharedSettings)

		/** The Category to expand in the Filter Menu */
		SLATE_ARGUMENT(TSharedPtr<FFilterCategory>, CategoryToExpand)

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

	/**
	 * Register a filter as a "type filter" so it appears as an entry in the "Type Filters" section of the Add Filter menu
	 * rather than under "Other Filters". The filter will appear under its primary category (set in the Filter)
	 * and each additional category in the "Add Filter" menu. Toggling from any category activates the same filter instance,
	 * providing a shared state.
	 */
	void AddTypeFilter(const TSharedRef<FFilterBase<SceneOutliner::FilterBarType>>& InFilter,
		TArrayView<const TSharedPtr<FFilterCategory>> InAdditionalCategories = {});
	
	/** Override from the AssetFilterBar to allow non-FCustomClassFilterData to populate the TypeFilter section */
	virtual void PopulateAddFilterMenu(UToolMenu* Menu, TSharedPtr<FFilterCategory> MenuExpansion, FOnFilterAssetType OnFilterAssetType) override;

	virtual void SaveSettings() override;
	virtual void LoadSettings() override;

	/** Handler for when a custom text filter is created */
	virtual void OnCreateCustomTextFilter(const FCustomTextFilterData& InFilterData, bool bApplyFilter) override;
	/** Handler for when a custom text filter is modified */
	virtual void OnModifyCustomTextFilter(const FCustomTextFilterData& InFilterData, TSharedPtr<ICustomTextFilter<SceneOutliner::FilterBarType>> InFilter) override;
	/** Handler for when a custom text filter is deleted */
	virtual void OnDeleteCustomTextFilter(const TSharedPtr<ICustomTextFilter<SceneOutliner::FilterBarType>> InFilter) override;

protected:
	virtual UAssetFilterBarContext* CreateAssetFilterBarContext() override;

private:
	/**
	 * Unified representation of a single "Type Filters" menu category.
	 * A category may contain FCustomClassFilterData entries, FFilterBase entries, or both.
	 */
	struct FTypeFilterCategoryEntry
	{
		/** Extension hook and heading used when this category is rendered as a section or submenu */
		FName SectionExtensionHook;
		FText SectionHeading;

		/** Asset class filters managed by the asset filter bar infrastructure */
		TArray<TSharedPtr<FCustomClassFilterData>> Classes;

		/** Type filters managed via AllFrontendFilters */
		TArray<TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>> Filters;
	};
	
	/** Handler for when another Filter Bar using shared settings creates a custom text filter */
	void OnExternalCustomTextFilterCreated(TSharedPtr<SWidget> BroadcastingFilterBar);
	
	/** Empty our list of custom text filters, and load from the given config */
	void LoadCustomTextFilters(const FFilterBarSettings* FilterBarConfig);

	/** Find the custom text filter corresponding to the specified state, and restore it's state to what is specified
	 *  @return True if the filter was restored successfully, false if not
	 */
	bool RestoreCustomTextFilterState(const FCustomTextFilterState& InFilterState);

	void PopulateTypeFilterMap(const FOnFilterAssetType& OnFilterAssetType, TMap<TSharedPtr<FFilterCategory>, FTypeFilterCategoryEntry>& TypeFilterMap);

	/** Populate a submenu section from an explicit filter list, bypassing GetCategory() matching. */
	void CreateTypeFiltersMenuSection(UToolMenu* InMenu, TArray<TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>> InFilters);

	/** Toggle all filters in the explicit list on or off (analogous to FrontendFilterCategoryClicked). */
	void TypeFilterCategoryClicked(TArray<TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>> InFilters);

	/** Return the tri-state check state for the category toggle button (analogous to IsFrontendFilterCategoryChecked). */
	ECheckBoxState IsTypeFilterCategoryChecked(TArray<TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>> InFilters) const;
	
private:
	/** The event that executes when a custom text filter is created/modified/deleted in any filter list that is using Shared Settings */
	static FCustomTextFilterEvent CustomTextFilterEvent;

	/** An identifier shared by all SSceneOutlinerFilterBars, used to save and load settings common to every instance */
	static const FName SharedIdentifier;

	/** Whether this Filter bar wants to load/save from the settings common to all instances */
	bool bUseSharedSettings;

	TSharedPtr<FFilterCategory> CategoryToExpand;

	/**
	 * Filters registered via AddTypeFilter, used when building the 'Type Filters' menu section. Can be populated with extra categories
	 * registered for specific type filters. PopulateTypeFilterMap distributes each filter across its primary category plus any entries
	 * in the value array.
	 */
	TMap<TSharedPtr<FFilterBase<SceneOutliner::FilterBarType>>, TArray<TSharedPtr<FFilterCategory>>> TypeFilters;
};