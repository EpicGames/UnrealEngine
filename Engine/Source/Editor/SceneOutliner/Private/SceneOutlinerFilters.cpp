// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerFilters.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Textures/SlateIcon.h"
#include "UObject/UnrealNames.h"

void FSceneOutlinerFilterInfo::InitFilter(TSharedPtr<FSceneOutlinerFilters> InFilters)
{
	Filters = InFilters;

	ApplyFilter(bActive);
}

void FSceneOutlinerFilterInfo::AddMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.AddMenuEntry(
		FilterTitle,
		FilterTooltip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw( this, &FSceneOutlinerFilterInfo::ToggleFilterActive ),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw( this, &FSceneOutlinerFilterInfo::IsFilterActive )
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
}

void FSceneOutlinerFilterInfo::ApplyFilter(bool bInActive)
{
	if ( !Filter.IsValid() )
	{
		Filter = Factory.Execute();
	}

	if ( bInActive )
	{			
		Filters.Pin()->Add( Filter );
	}
	else
	{
		Filters.Pin()->Remove( Filter );
	}
}

void FSceneOutlinerFilterInfo::ToggleFilterActive()
{
	bActive = !bActive;

	ApplyFilter(bActive);

	OnToggleEvent.Broadcast(bActive);
}

bool FSceneOutlinerFilterInfo::IsFilterActive() const
{
	return bActive;
}
