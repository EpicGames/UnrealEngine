// Copyright Epic Games, Inc. All Rights Reserved.


#include "SceneOutlinerGutter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Widgets/Views/STreeView.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"
#include "SortHelper.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerGutter"

bool FSceneOutlinerVisibilityCache::RecurseChildren(const ISceneOutlinerTreeItem& Item) const
{
	if (const bool* Info = VisibilityInfo.Find(&Item))
	{
		return *Info;
	}
	else
	{
		bool bIsVisible = false;
		for (const auto& ChildPtr : Item.GetChildren())
		{
			auto Child = ChildPtr.Pin();
			if (Child.IsValid() && GetVisibility(*Child))
			{
				bIsVisible = true;
				break;
			}
		}
		VisibilityInfo.Add(&Item, bIsVisible);
		
		return bIsVisible;
	}
}

bool FSceneOutlinerVisibilityCache::GetVisibility(const ISceneOutlinerTreeItem& Item) const
{
	if (Item.HasVisibilityInfo())
	{
		if (const bool* Info = VisibilityInfo.Find(&Item))
		{
			return *Info;
		}
		else
		{
			const bool bIsVisible = Item.GetVisibility();
			VisibilityInfo.Add(&Item, bIsVisible);
			return bIsVisible;
		}
	}
	else
	{
		return RecurseChildren(Item);
	}
}

class FVisibilityDragDropOp : public FDragDropOperation
{
public:
	
	DRAG_DROP_OPERATOR_TYPE(FVisibilityDragDropOp, FDragDropOperation)

	/** Flag which defines whether to hide destination items or not */
	bool bHidden;

	/** Undo transaction stolen from the gutter which is kept alive for the duration of the drag */
	TUniquePtr<FScopedTransaction> UndoTransaction;

	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}

	/** Create a new drag and drop operation out of the specified flag */
	static TSharedRef<FVisibilityDragDropOp> New(const bool _bHidden, TUniquePtr<FScopedTransaction>& ScopedTransaction)
	{
		TSharedRef<FVisibilityDragDropOp> Operation = MakeShareable(new FVisibilityDragDropOp);

		Operation->bHidden = _bHidden;
		Operation->UndoTransaction = MoveTemp(ScopedTransaction);

		Operation->Construct();
		return Operation;
	}
};

//
// SVisibilityWidget
//

void SVisibilityWidget::Construct(const FArguments& InArgs, TWeakPtr<FSceneOutlinerGutter> InWeakColumn, TWeakPtr<ISceneOutliner> InWeakOutliner, TWeakPtr<ISceneOutlinerTreeItem> InWeakTreeItem, const STableRow<FSceneOutlinerTreeItemPtr>* InRow)
{
	WeakTreeItem = InWeakTreeItem;
	WeakOutliner = InWeakOutliner;
	WeakColumn = InWeakColumn;

	Row = InRow;

	SImage::Construct(
		SImage::FArguments()
		.IsEnabled(this, &SVisibilityWidget::IsEnabled)
		.ColorAndOpacity(this, &SVisibilityWidget::GetForegroundColor)
		.Image(this, &SVisibilityWidget::GetBrush)
	);


	static const FName NAME_VisibleHoveredBrush = TEXT("Level.VisibleHighlightIcon16x");
	static const FName NAME_VisibleNotHoveredBrush = TEXT("Level.VisibleIcon16x");
	static const FName NAME_NotVisibleHoveredBrush = TEXT("Level.NotVisibleHighlightIcon16x");
	static const FName NAME_NotVisibleNotHoveredBrush = TEXT("Level.NotVisibleIcon16x");

	VisibleHoveredBrush = FAppStyle::Get().GetBrush(NAME_VisibleHoveredBrush);
	VisibleNotHoveredBrush = FAppStyle::Get().GetBrush(NAME_VisibleNotHoveredBrush);

	NotVisibleHoveredBrush = FAppStyle::Get().GetBrush(NAME_NotVisibleHoveredBrush);
	NotVisibleNotHoveredBrush = FAppStyle::Get().GetBrush(NAME_NotVisibleNotHoveredBrush);

}

FReply SVisibilityWidget::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Handled().BeginDragDrop(FVisibilityDragDropOp::New(!IsVisible(), UndoTransaction));
	}
	else
	{
		return FReply::Unhandled();
	}
}

/** If a visibility drag drop operation has entered this widget, set its item to the new visibility state */
void SVisibilityWidget::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	auto VisibilityOp = DragDropEvent.GetOperationAs<FVisibilityDragDropOp>();
	if (VisibilityOp.IsValid())
	{
		SetIsVisible(!VisibilityOp->bHidden);
	}
}

FReply SVisibilityWidget::HandleClick()
{
	if (!IsEnabled())
	{
		return FReply::Unhandled();
	}

	auto Outliner = WeakOutliner.Pin();
	auto TreeItem = WeakTreeItem.Pin();
	auto Column = WeakColumn.Pin();
		
	if (!Outliner.IsValid() || !TreeItem.IsValid() || !Column.IsValid())
	{
		return FReply::Unhandled();
	}

	// Open an undo transaction
	UndoTransaction.Reset(new FScopedTransaction(LOCTEXT("SetOutlinerItemVisibility", "Set Item Visibility")));

	const auto& Tree = Outliner->GetTree();

	const bool bVisible = !IsVisible();

	// We operate on all the selected items if the specified item is selected
	if (Tree.IsItemSelected(TreeItem.ToSharedRef()))
	{
		for (auto& SelectedItem : Tree.GetSelectedItems())
		{
			if (IsVisible(SelectedItem, Column) != bVisible)
			{
				OnSetItemVisibility(*SelectedItem, bVisible);
			}		
		}

		GEditor->RedrawAllViewports();
	}
	else
	{
		SetIsVisible(bVisible);
	}

	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FReply SVisibilityWidget::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return HandleClick();
}

/** Called when the mouse button is pressed down on this widget */
FReply SVisibilityWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	return HandleClick();
}

/** Process a mouse up message */
FReply SVisibilityWidget::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		UndoTransaction.Reset();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
void SVisibilityWidget::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	UndoTransaction.Reset();
}

/** Get the brush for this widget */
const FSlateBrush* SVisibilityWidget::GetBrush() const
{
	if (IsVisible())
	{
		return IsHovered() ? VisibleHoveredBrush : VisibleNotHoveredBrush;
	}
	else
	{
		return IsHovered() ? NotVisibleHoveredBrush : NotVisibleNotHoveredBrush;
	}
}

FSlateColor SVisibilityWidget::GetForegroundColor() const
{

	auto Outliner = WeakOutliner.Pin();
	auto TreeItem = WeakTreeItem.Pin();

	const bool bIsSelected = Outliner->GetTree().IsItemSelected(TreeItem.ToSharedRef());

	// make the foreground brush transparent if it is not selected and it is visible
	if (IsVisible() && !Row->IsHovered() && !bIsSelected)
	{
		return FLinearColor::Transparent;
	}
	else if (IsHovered() && !bIsSelected)
	{
		return FAppStyle::Get().GetSlateColor("Colors.ForegroundHover");
	}

	return FSlateColor::UseForeground();
}

/** Check if the specified item is visible */
bool SVisibilityWidget::IsVisible(const FSceneOutlinerTreeItemPtr& Item, const TSharedPtr<FSceneOutlinerGutter>& Column)
{
	return Column.IsValid() && Item.IsValid() ? Column->IsItemVisible(*Item) : false;
}

/** Check if our wrapped tree item is visible */
bool SVisibilityWidget::IsVisible() const
{
	return SVisibilityWidget::IsVisible(WeakTreeItem.Pin(), WeakColumn.Pin());
}

/** Set the item this widget is responsible for to be hidden or shown */
void SVisibilityWidget::SetIsVisible(const bool bVisible)
{
	TSharedPtr<ISceneOutlinerTreeItem> TreeItem = WeakTreeItem.Pin();
	TSharedPtr<ISceneOutliner> Outliner = WeakOutliner.Pin();

	if (TreeItem.IsValid() && Outliner.IsValid() && IsVisible() != bVisible)
	{
		OnSetItemVisibility(*TreeItem, bVisible);
			
		Outliner->Refresh();

		GEditor->RedrawAllViewports();
	}
}

void SVisibilityWidget::OnSetItemVisibility(ISceneOutlinerTreeItem& Item, const bool bNewVisibility)
{
	// Apply the same visibility to the children
	Item.OnVisibilityChanged(bNewVisibility);

	if (ShouldPropagateVisibilityChangeOnChildren())
	{
		for (auto& ChildPtr : Item.GetChildren())
		{
			auto Child = ChildPtr.Pin();
			if (Child.IsValid())
			{
				OnSetItemVisibility(*Child, bNewVisibility);
			}
		}
	}
}

FSceneOutlinerGutter::FSceneOutlinerGutter(ISceneOutliner& Outliner)
{
	WeakOutliner = StaticCastSharedRef<ISceneOutliner>(Outliner.AsShared());
}

void FSceneOutlinerGutter::Tick(double InCurrentTime, float InDeltaTime)
{
	VisibilityCache.VisibilityInfo.Empty();
}

FName FSceneOutlinerGutter::GetColumnID()
{
	return GetID();
}

SHeaderRow::FColumn::FArguments FSceneOutlinerGutter::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(GetDisplayName())
		.HeaderContentPadding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("Level.VisibleIcon16x"))
		];
}

const TSharedRef<SWidget> FSceneOutlinerGutter::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (TreeItem->ShouldShowVisibilityState())
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SVisibilityWidget, SharedThis(this), WeakOutliner, TreeItem, &Row)
				.ToolTipText(LOCTEXT("SceneOutlinerVisibilityToggleTooltip", "Toggles the visibility of this object in the level editor."))
			];
	}
	return SNullWidget::NullWidget;
}

void FSceneOutlinerGutter::SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const
{
	FSceneOutlinerSortHelper<int32, bool>()
		/** Sort by type first */
		.Primary([this](const ISceneOutlinerTreeItem& Item){ return WeakOutliner.Pin()->GetMode()->GetTypeSortPriority(Item); }, SortMode)
		/** Then by visibility */
		.Secondary([](const ISceneOutlinerTreeItem& Item) {return FSceneOutlinerVisibilityCache().GetVisibility(Item); },					SortMode)
		.Sort(RootItems);
}

#undef LOCTEXT_NAMESPACE

