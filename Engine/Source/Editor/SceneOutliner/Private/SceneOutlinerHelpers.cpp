// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerHelpers.h"
#include "ActorTreeItem.h"
#include "ActorDescTreeItem.h"
#include "ActorFolderTreeItem.h"
#include "ComponentTreeItem.h"
#include "EditorActorFolders.h"
#include "EditorClassUtils.h"
#include "EditorFolderUtils.h"
#include "EditorModeManager.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "LevelTreeItem.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerFwd.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "WorldTreeItem.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "UObject/Package.h"

static int32 GSceneOutlinerAutoRepresentingWorldNetMode = NM_Client;
static FAutoConsoleVariableRef CVarAutoRepresentingWorldNetMode(
	TEXT("SceneOutliner.AutoRepresentingWorldNetMode"),
	GSceneOutlinerAutoRepresentingWorldNetMode,
	TEXT("The preferred NetMode of the world shown in the scene outliner when the 'Auto' option is chosen: 0=Standalone, 1=DedicatedServer, 2=ListenServer, 3=Client"));


#define LOCTEXT_NAMESPACE "SceneOutlinerHelpers"

namespace SceneOutliner
{
	FString FSceneOutlinerHelpers::GetExternalPackageName(const ISceneOutlinerTreeItem& TreeItem)
	{
		if (const FActorTreeItem* ActorItem = TreeItem.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				if (Actor->IsPackageExternal())
				{
					return Actor->GetExternalPackage()->GetName();
				}

			}
		}
		else if (const FActorFolderTreeItem* ActorFolderItem = TreeItem.CastTo<FActorFolderTreeItem>())
		{
			if (const UActorFolder* ActorFolder = ActorFolderItem->GetActorFolder())
			{
				if (ActorFolder->IsPackageExternal())
				{
					return ActorFolder->GetExternalPackage()->GetName();
				}
			}
		}
		else if (const FActorDescTreeItem* ActorDescItem = TreeItem.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				return ActorDescInstance->GetActorPackage().ToString();
			}
		}

		return FString();
	}
	
	UPackage* FSceneOutlinerHelpers::GetExternalPackage(const ISceneOutlinerTreeItem& TreeItem)
	{
		if (const FActorTreeItem* ActorItem = TreeItem.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				if (Actor->IsPackageExternal())
				{
					return Actor->GetExternalPackage();
				}
			}
		}
		else if (const FActorFolderTreeItem* ActorFolderItem = TreeItem.CastTo<FActorFolderTreeItem>())
		{
			if (const UActorFolder* ActorFolder = ActorFolderItem->GetActorFolder())
			{
				if (ActorFolder->IsPackageExternal())
				{
					return ActorFolder->GetExternalPackage();
				}
			}
		}
		else if (const FActorDescTreeItem* ActorDescItem = TreeItem.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				return FindPackage(nullptr, *ActorDescInstance->GetActorPackage().ToString());
			}
		}

		return nullptr;
	}
	
	void FSceneOutlinerHelpers::ChooseRepresentingWorld(TWeakObjectPtr<UWorld>& OutWorld, const TWeakObjectPtr<UWorld>& InSpecifiedWorldToDisplay, const TWeakObjectPtr<UWorld>& InUserChosenWorld)
	{
		// Select a world to represent

		TWeakObjectPtr<UWorld> RepresentingWorld = nullptr;

		// If a specified world was provided, represent it
		if (InSpecifiedWorldToDisplay.IsValid())
		{
			RepresentingWorld = InSpecifiedWorldToDisplay.Get();
		}

		// check if the user-chosen world is valid and in the editor contexts

		if (!RepresentingWorld.IsValid() && InUserChosenWorld.IsValid())
		{
			for (const FWorldContext& Context : GEngine->GetWorldContexts())
			{
				if (InUserChosenWorld.Get() == Context.World())
				{
					RepresentingWorld = InUserChosenWorld.Get();
					break;
				}
			}
		}

		// If the user did not manually select a world, try to pick the most suitable world context
		if (!RepresentingWorld.IsValid())
		{
			// Ideally we want a PIE world that is standalone or the first client, unless the preferred NetMode is overridden by CVar
			int32 LowestPIEInstanceSeen = MAX_int32;
			for (const FWorldContext& Context : GEngine->GetWorldContexts())
			{
				UWorld* World = Context.World();
				if (World && Context.WorldType == EWorldType::PIE)
				{
					if (World->GetNetMode() == NM_Standalone)
					{
						RepresentingWorld = World;
						break;
					}
					else if ((World->GetNetMode() == ENetMode(GSceneOutlinerAutoRepresentingWorldNetMode)) && (Context.PIEInstance < LowestPIEInstanceSeen))
					{
						RepresentingWorld = World;
						LowestPIEInstanceSeen = Context.PIEInstance;
					}
				}
			}
		}

		if (!RepresentingWorld.IsValid() && FModuleManager::Get().IsModuleLoaded("LevelEditor"))
		{
			// If there is still no world, we query the Level Editor, which prefers the PIE world over the Editor world
			const TWeakPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();

			if (const TSharedPtr<ILevelEditor> LevelEditorPin = LevelEditor.Pin())
			{
				RepresentingWorld = LevelEditorPin->GetEditorModeManager().GetWorld();
			}
		}

		OutWorld = RepresentingWorld;
	}

	TSharedPtr<SWidget> FSceneOutlinerHelpers::GetClassHyperlink(UObject* InObject)
	{
		if (InObject)
		{
			if (UClass* Class = InObject->GetClass())
			{
				// Always show blueprints
				const bool bIsBlueprintClass = UBlueprint::GetBlueprintFromClass(Class) != nullptr;

				// Also show game or game plugin native classes (but not engine classes as that makes the scene outliner pretty noisy)
				bool bIsGameClass = false;
				if (!bIsBlueprintClass)
				{
					UPackage* Package = Class->GetOutermost();
					const FString ModuleName = FPackageName::GetShortName(Package->GetFName());

					FModuleStatus PackageModuleStatus;
					if (FModuleManager::Get().QueryModule(*ModuleName, /*out*/ PackageModuleStatus))
					{
						bIsGameClass = PackageModuleStatus.bIsGameModule;
					}
				}

				if (bIsBlueprintClass || bIsGameClass)
				{
					FEditorClassUtils::FSourceLinkParams SourceLinkParams;
					SourceLinkParams.Object = InObject;
					SourceLinkParams.bUseDefaultFormat = true;

					return FEditorClassUtils::GetSourceLink(Class, SourceLinkParams);
				}
			}
		}

		return nullptr;
	}

	void FSceneOutlinerHelpers::PopulateExtraSearchStrings(const ISceneOutlinerTreeItem& TreeItem, TArray< FString >& OutSearchStrings)
	{
		// For components, we want them to be searchable by the actor name if they request so. This is so you can search by actors in component
		// pickers without the actual components themselves being filtered out.
		if (const FComponentTreeItem* ComponentTreeItem = TreeItem.CastTo<FComponentTreeItem>())
		{
			if (ComponentTreeItem->GetSearchComponentByActorName())
			{
				if (const UActorComponent* Component = ComponentTreeItem->Component.Get())
				{
					if (const AActor* Owner = Component->GetOwner())
					{
						constexpr bool bCreateIfNone = false;
						OutSearchStrings.Add(Owner->GetActorLabel(bCreateIfNone));
					}
					
				}
			}
		}
	}

	void FSceneOutlinerHelpers::RenameFolder(const FFolder& InFolder, const FText& NewFolderName, UWorld* World)
	{
		FEditorFolderUtils::RenameFolder(InFolder, NewFolderName, World);
	}

	bool FSceneOutlinerHelpers::ValidateFolderName(const FFolder& InFolder, UWorld* World, const FText& InLabel, FText& OutErrorMessage)
	{
		const FText TrimmedLabel = FText::TrimPrecedingAndTrailing(InLabel);

		if (TrimmedLabel.IsEmpty())
		{
			OutErrorMessage = LOCTEXT("RenameFailed_LeftBlank", "Names cannot be left blank");
			return false;
		}

		if (TrimmedLabel.ToString().Len() >= NAME_SIZE)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("CharCount"), NAME_SIZE);
			OutErrorMessage = FText::Format(LOCTEXT("RenameFailed_TooLong", "Names must be less than {CharCount} characters long."), Arguments);
			return false;
		}

		const FString LabelString = TrimmedLabel.ToString();
		if (InFolder.GetLeafName().ToString() == LabelString)
		{
			return true;
		}

		int32 Dummy = 0;
		if (LabelString.FindChar('/', Dummy) || LabelString.FindChar('\\', Dummy))
		{
			OutErrorMessage = LOCTEXT("RenameFailed_InvalidChar", "Folder names cannot contain / or \\.");
			return false;
		}

		// Validate that this folder doesn't exist already
		FName NewPath = InFolder.GetParent().GetPath();
		if (NewPath.IsNone())
		{
			NewPath = FName(*LabelString);
		}
		else
		{
			NewPath = FName(*(NewPath.ToString() / LabelString));
		}
		FFolder NewFolder(InFolder.GetRootObject(), NewPath);

		if (World && FActorFolders::Get().ContainsFolder(*World, NewFolder))
		{
			OutErrorMessage = LOCTEXT("RenameFailed_AlreadyExists", "A folder with this name already exists at this level");
			return false;
		}

		return true;
	}

	bool FSceneOutlinerHelpers::IsFolderCurrent(const FFolder& InFolder, UWorld* World)
	{
		if (World)
		{
			return FActorFolders::Get().GetActorEditorContextFolder(*World) == InFolder;
		}
		return false;
	}

	TWeakObjectPtr<AActor> FSceneOutlinerHelpers::GetActorFromOutlinerTreeItem(const ISceneOutlinerTreeItem& TreeItem, const TSharedPtr<ISceneOutliner>& Outliner)
	{
		TWeakObjectPtr<AActor> Actor;

		if (const FActorTreeItem* ActorItem = TreeItem.CastTo<FActorTreeItem>())
		{
			Actor = ActorItem->Actor;
		}
		// Early-out if we can tell that this tree item isn't a TEDS Tree Item
		else if (TreeItem.IsA<FComponentTreeItem>() || TreeItem.IsA<FFolderTreeItem>() || TreeItem.IsA<FActorDescTreeItem>()
			|| TreeItem.IsA<FActorFolderTreeItem>() || TreeItem.IsA<FLevelTreeItem>() || TreeItem.IsA<FWorldTreeItem>())
		{
			return Actor;
		}

		if (!Actor.IsValid() && Outliner)
		{
			using namespace UE::Editor::DataStorage;
			if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				const RowHandle OutlinerRow = Storage->LookupMappedRow(UE::Editor::Outliner::OutlinerMappingDomain,
					FMapKey(Outliner->GetOutlinerIdentifier()));
				if (const FTedsOutlinerRowHandleDealiaserColumn* Dealiaser = Storage->GetColumn<FTedsOutlinerRowHandleDealiaserColumn>(OutlinerRow);
					Dealiaser && Dealiaser->GetRowHandle.IsBound())
				{
					const RowHandle ItemRow = Dealiaser->GetRowHandle.Execute(TreeItem);
					if (const FTypedElementUObjectColumn* ObjectColumn = Storage->GetColumn<FTypedElementUObjectColumn>(ItemRow))
					{
						Actor = Cast<AActor>(ObjectColumn->Object.Get());
					}
				}
			}
		}
		
		return Actor;
	}
}

#undef LOCTEXT_NAMESPACE