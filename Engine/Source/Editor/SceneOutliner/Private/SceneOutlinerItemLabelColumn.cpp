// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerItemLabelColumn.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/SOverlay.h"
#include "Engine/GameViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "ISceneOutlinerMode.h"
#include "SceneOutlinerDragDrop.h"
#include "SceneOutlinerHelpers.h"
#include "Widgets/Views/SListView.h"
#include "SortHelper.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerItemLabelColumn"

FName FSceneOutlinerItemLabelColumn::GetColumnID()
{
	return GetID();
}

SHeaderRow::FColumn::FArguments FSceneOutlinerItemLabelColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FillWidth( 5.0f );
}
const TSharedRef<SWidget> FSceneOutlinerItemLabelColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	ISceneOutliner* Outliner = WeakSceneOutliner.Pin().Get();
	check(Outliner);
	return TreeItem->GenerateLabelWidget(*Outliner, Row);
}

void FSceneOutlinerItemLabelColumn::PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const
{
	OutSearchStrings.Add(Item.GetDisplayString());
	SceneOutliner::FSceneOutlinerHelpers::PopulateExtraSearchStrings(Item, OutSearchStrings);
}

void FSceneOutlinerItemLabelColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const
{
	typedef FSceneOutlinerSortHelper<int32, SceneOutliner::FNumericStringWrapper> FSort;

	FSort()
		.Primary([this](const ISceneOutlinerTreeItem& Item){ return WeakSceneOutliner.Pin()->GetTypeSortPriority(Item); },			SortMode)
		.Secondary([](const ISceneOutlinerTreeItem& Item){ return SceneOutliner::FNumericStringWrapper(Item.GetDisplayString()); }, SortMode)
		.Sort(OutItems);
}

#undef LOCTEXT_NAMESPACE
