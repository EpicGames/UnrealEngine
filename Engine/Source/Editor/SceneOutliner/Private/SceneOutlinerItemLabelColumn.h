// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerPublicTypes.h"
#include "ISceneOutlinerColumn.h"

template<typename ItemType> class STableRow;

/** A column for the SceneOutliner that displays the item label */
class FSceneOutlinerItemLabelColumn : public ISceneOutlinerColumn
{

public:
	FSceneOutlinerItemLabelColumn(ISceneOutliner& SceneOutliner) : WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared())) {}
	
	virtual ~FSceneOutlinerItemLabelColumn() = default;

	static FName GetID() { return FSceneOutlinerBuiltInColumnTypes::Label(); }

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation

	virtual FName GetColumnID() override;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	
	virtual const TSharedRef< SWidget > ConstructRowWidget( FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row ) override;

	virtual void PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const override;

	virtual bool SupportsSorting() const override { return true; }

	virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const override;
	
	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////

private:
	TWeakPtr<ISceneOutliner> WeakSceneOutliner;

};
