// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Application/SlateApplication.h"
#include "SSceneOutliner.h"
#include "Algo/Find.h"

#include "SceneOutlinerActorInfoColumn.h"
#include "SceneOutlinerGutter.h"
#include "SceneOutlinerItemLabelColumn.h"
#include "SceneOutlinerSourceControlColumn.h"
#include "SceneOutlinerPinnedColumn.h"
#include "SceneOutlinerTextInfoColumn.h"
#include "SceneOutlinerUnsavedColumn.h"
#include "SceneOutlinerPublicTypes.h"

#include "ActorPickingMode.h"
#include "ActorBrowsingMode.h"
#include "ActorBrowsingModeCommands.h"
#include "ActorTreeItem.h"
#include "ActorFolderTreeItem.h"
#include "ComponentTreeItem.h"
#include "ActorDescTreeItem.h"
#include "FolderTreeItem.h"
#include "WorldTreeItem.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/ContentBundle/ContentBundleEngineSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "Editor.h"
#include "ISourceControlModule.h"
#include "EditorModeManager.h"
#include "LevelEditor.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerModule"

/* FSceneOutlinerModule interface
 *****************************************************************************/

FSceneOutlinerModule::FSceneOutlinerModule()
: ColumnPermissionList(MakeShareable(new FNamePermissionList()))
{
	ColumnPermissionList->OnFilterChanged().AddLambda([this]() { ColumnPermissionListChanged.Broadcast(); });
}

void FSceneOutlinerModule::StartupModule()
{
	RegisterDefaultColumnType< FSceneOutlinerItemLabelColumn >(FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10, FCreateSceneOutlinerColumn(), false, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Label_Localized()));

	// Register builtin column types which are not active by default
	RegisterColumnType<FSceneOutlinerGutter>();
	RegisterColumnType<FTypeInfoColumn>();
	RegisterColumnType<FSceneOutlinerSourceControlColumn>();
	RegisterColumnType<FSceneOutlinerPinnedColumn>();
	RegisterColumnType<FSceneOutlinerActorUnsavedColumn>();

	// Register Commands
	FActorBrowsingModeCommands::Register();
}


void FSceneOutlinerModule::ShutdownModule()
{
	UnRegisterColumnType<FSceneOutlinerGutter>();
	UnRegisterColumnType<FSceneOutlinerItemLabelColumn>();
	UnRegisterColumnType<FTypeInfoColumn>();
	UnRegisterColumnType<FSceneOutlinerSourceControlColumn>();
	UnRegisterColumnType<FSceneOutlinerPinnedColumn>();
	UnRegisterColumnType<FSceneOutlinerActorUnsavedColumn>();

	// Un-Register Commands
	FActorBrowsingModeCommands::Unregister();
}

TSharedRef<ISceneOutliner> FSceneOutlinerModule::CreateSceneOutliner(const FSceneOutlinerInitializationOptions& InitOptions) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSceneOutlinerModule::CreateSceneOutliner);

	return SNew(SSceneOutliner, InitOptions)
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute());
}

TSharedRef<ISceneOutliner> FSceneOutlinerModule::CreateActorPicker(const FSceneOutlinerInitializationOptions& InInitOptions, const FOnActorPicked& OnActorPickedDelegate, TWeakObjectPtr<UWorld> SpecifiedWorld, bool bInHideLevelInstanceHierarchy) const
{
	FSceneOutlinerInitializationOptions InitOptions(InInitOptions);
	if (!InitOptions.ModeFactory.IsBound())
	{
		auto OnItemPicked = FOnSceneOutlinerItemPicked::CreateLambda([OnActorPickedDelegate](TSharedRef<ISceneOutlinerTreeItem> Item)
			{
				if (FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
				{
					if (ActorItem->IsValid())
					{
						OnActorPickedDelegate.ExecuteIfBound(ActorItem->Actor.Get());

					}
				}
			});

		FCreateSceneOutlinerMode ModeFactory = FCreateSceneOutlinerMode::CreateLambda([OnItemPicked, SpecifiedWorld, bInHideLevelInstanceHierarchy](SSceneOutliner* Outliner)
			{
				FActorModeParams Params;
				Params.SceneOutliner = Outliner;
				Params.SpecifiedWorldToDisplay = SpecifiedWorld;
				Params.bHideComponents = true;
				Params.bHideLevelInstanceHierarchy = bInHideLevelInstanceHierarchy;
				Params.bHideUnloadedActors = true;
				Params.bHideEmptyFolders = true;
				return new FActorPickingMode(Params, OnItemPicked);
			});

		InitOptions.ModeFactory = ModeFactory;
	}

	if (InitOptions.ColumnMap.Num() == 0)
	{
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn(), false, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Label_Localized()));
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::ActorInfo(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10, FCreateSceneOutlinerColumn(), true, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::ActorInfo_Localized()));
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Pinned(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Invisible, 5, FCreateSceneOutlinerColumn(), true, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Pinned_Localized()));
		CreateActorInfoColumns(InitOptions);
	}
	return CreateSceneOutliner(InitOptions);
}

TSharedRef<ISceneOutliner> FSceneOutlinerModule::CreateComponentPicker(const FSceneOutlinerInitializationOptions& InInitOptions, const FOnComponentPicked& OnComponentPickedDelegate, TWeakObjectPtr<UWorld> SpecifiedWorld) const
{
	auto OnItemPicked = FOnSceneOutlinerItemPicked::CreateLambda([OnComponentPickedDelegate](TSharedRef<ISceneOutlinerTreeItem> Item)
		{
			if (FComponentTreeItem* ComponentItem = Item->CastTo<FComponentTreeItem>())
			{
				if (ComponentItem->IsValid())
				{
					OnComponentPickedDelegate.ExecuteIfBound(ComponentItem->Component.Get());
				}
			}
		});

	FCreateSceneOutlinerMode ModeFactory = FCreateSceneOutlinerMode::CreateLambda([&OnItemPicked, &SpecifiedWorld](SSceneOutliner* Outliner)
		{
			FActorModeParams Params;
			Params.SceneOutliner = Outliner;
			Params.SpecifiedWorldToDisplay = SpecifiedWorld;
			Params.bHideComponents = false;
			Params.bHideActorWithNoComponent = true;
			Params.bHideLevelInstanceHierarchy = true;
			Params.bHideUnloadedActors = true;
			Params.bHideEmptyFolders = true;
			Params.bSearchComponentsByActorName = true;
			return new FActorPickingMode(Params, OnItemPicked);
		});

	FSceneOutlinerInitializationOptions InitOptions(InInitOptions);
	InitOptions.ModeFactory = ModeFactory;
	if (InitOptions.ColumnMap.Num() == 0)
	{
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn(), false, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Label_Localized()));
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::ActorInfo(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10, FCreateSceneOutlinerColumn(), true, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::ActorInfo_Localized()));
		CreateActorInfoColumns(InitOptions);
	}
	return CreateSceneOutliner(InitOptions);
}

TSharedRef< ISceneOutliner > FSceneOutlinerModule::CreateActorBrowser(const FSceneOutlinerInitializationOptions& InInitOptions, TWeakObjectPtr<UWorld> SpecifiedWorld) const
{
	FCreateSceneOutlinerMode ModeFactory = FCreateSceneOutlinerMode::CreateLambda([&SpecifiedWorld](SSceneOutliner* Outliner)
		{
			return new FActorBrowsingMode(Outliner, SpecifiedWorld);
		});

	FSceneOutlinerInitializationOptions InitOptions(InInitOptions);
	InitOptions.ModeFactory = ModeFactory;

	if (InitOptions.ColumnMap.Num() == 0)
	{
		CreateActorBrowserColumns(InitOptions);
	}

	return CreateSceneOutliner(InitOptions);
}

TSharedPtr<ISceneOutliner> FSceneOutlinerModule::CreateCustomRegisteredOutliner(FName ID, FSceneOutlinerInitializationOptions InInitOptions)
{
	FSceneOutlinerFactory* FoundInitOptionsFactory = CustomOutlinerFactories.Find(ID);

	if(!FoundInitOptionsFactory)
	{
		return nullptr;
	}

	return FoundInitOptionsFactory->Execute(InInitOptions);
}

void FSceneOutlinerModule::RegisterCustomSceneOutlinerFactory(FName ID,
	FSceneOutlinerFactory InOutlinerFactory)
{
	CustomOutlinerFactories.Add(ID, InOutlinerFactory);
}

void FSceneOutlinerModule::UnregisterCustomSceneOutlinerFactory(FName ID)
{
	CustomOutlinerFactories.Remove(ID);
}

bool FSceneOutlinerModule::IsCustomSceneOutlinerFactoryRegistered(FName ID)
{
	return CustomOutlinerFactories.Contains(ID);
}

void FSceneOutlinerModule::CreateActorBrowserColumns(FSceneOutlinerInitializationOptions& InInitOptions, UWorld* InWorld) const
{
	if(!InWorld)
	{
		// Query the Level Editor to get the correct world based on context
		TWeakPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
		if (TSharedPtr<ILevelEditor> LevelEditorPin = LevelEditor.Pin())
		{
			InWorld = LevelEditorPin->GetEditorModeManager().GetWorld();
		}
	}

	InInitOptions.UseDefaultColumns();
	
	InInitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Gutter(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn(), true, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Gutter_Localized()));
	InInitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::ActorInfo(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 20, FCreateSceneOutlinerColumn(), true, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::ActorInfo_Localized()));

	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	// The source control column should be visible by default in source-controlled levels using external actors
	ESceneOutlinerColumnVisibility SourceControlColumnVisibility = (SourceControlModule.IsEnabled() && InWorld && InWorld->PersistentLevel->IsUsingExternalActors()) ? ESceneOutlinerColumnVisibility::Visible : ESceneOutlinerColumnVisibility::Invisible;
	InInitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::SourceControl(), FSceneOutlinerColumnInfo(SourceControlColumnVisibility, 95, FCreateSceneOutlinerColumn(), true, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::SourceControl_Localized()));

	// We want the unsaved column to be visible by default in partitioned levels
	ESceneOutlinerColumnVisibility UnsavedColumnVisibility = (InWorld && InWorld->IsPartitionedWorld()) ? ESceneOutlinerColumnVisibility::Visible : ESceneOutlinerColumnVisibility::Invisible;
	InInitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Unsaved(), FSceneOutlinerColumnInfo(UnsavedColumnVisibility, 1, FCreateSceneOutlinerColumn(), true, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Unsaved_Localized()));

	CreateActorInfoColumns(InInitOptions, InWorld);

	CreateActorBrowserColumnsDelegate.Broadcast(InInitOptions, InWorld);
}

void FSceneOutlinerModule::CreateActorInfoColumns(FSceneOutlinerInitializationOptions& InInitOptions, UWorld *WorldPtr) const
{
	FGetTextForItem MobilityInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			AActor* Actor = ActorItem->Actor.Get();
			if (!Actor)
			{
				return FString();
			}

			FString Result;
			USceneComponent* RootComponent = Actor->GetRootComponent();
			if (RootComponent)
			{
				if (RootComponent->Mobility == EComponentMobility::Static)
				{
					Result = FString(TEXT("Static"));
				}
				if (RootComponent->Mobility == EComponentMobility::Stationary)
				{
					Result = FString(TEXT("Stationary"));
				}
				else if (RootComponent->Mobility == EComponentMobility::Movable)
				{
					Result = FString(TEXT("Movable"));
				}
			}
			return Result;
		}
		return FString();
	});

	FGetTextForItem LayerInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		FString Result;
		
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			AActor* Actor = ActorItem->Actor.Get();
			if (!Actor)
			{
				return FString();
			}

			for (const auto& Layer : Actor->Layers)
			{
				if (Result.Len())
				{
					Result += TEXT(", ");
				}

				Result += Layer.ToString();
			}
		}

		return Result;
	});
	
	FGetTextForItem DataLayerInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		TStringBuilder<128> Builder;
		TSet<FString> DataLayerShortNames;

		auto BuildDataLayers = [&Builder, &DataLayerShortNames](const auto& DataLayerInstances, bool bPartOfOtherLevel)
		{
			if (const UDataLayerInstance* const* ExternalDataLayerInstance = !bPartOfOtherLevel ? Algo::FindByPredicate(DataLayerInstances, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsA<UExternalDataLayerInstance>(); }) : nullptr)
			{
				Builder += (*ExternalDataLayerInstance)->GetDataLayerShortName();
			}

			for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
			{
				if (!DataLayerInstance->IsA<UExternalDataLayerInstance>())
				{
					bool bIsAlreadyInSet = false;
					DataLayerShortNames.Add(DataLayerInstance->GetDataLayerShortName(), &bIsAlreadyInSet);
					if (!bIsAlreadyInSet)
					{
						if (Builder.Len())
						{
							Builder += TEXT(", ");
						}
						// Put a '*' in front of DataLayers that are not part of of the main world
						if (bPartOfOtherLevel)
						{
							Builder += "*";
						}
						Builder += DataLayerInstance->GetDataLayerShortName();
					}
				}
			}
		};

		auto BuildDataLayersWithContext = [BuildDataLayers](const ISceneOutlinerTreeItem& Item, bool bUseLevelContext)
		{
			if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
			{
				if (AActor* Actor = ActorItem->Actor.Get())
				{
					BuildDataLayers(bUseLevelContext ? Actor->GetDataLayerInstancesForLevel() : Actor->GetDataLayerInstances(), bUseLevelContext);
				}
			}
			else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
			{
				if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle; ActorDescInstance && !ActorDescInstance->GetDataLayerInstanceNames().IsEmpty())
				{
					if (const UActorDescContainerInstance* ActorDescContainerInstance = ActorDescInstance->GetContainerInstance())
					{
						const UWorld* OwningWorld = ActorDescContainerInstance->GetTopWorldPartition()->GetWorld();
						if (const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(OwningWorld))
						{
							TSet<const UDataLayerInstance*> DataLayerInstances;
							DataLayerInstances.Append(DataLayerManager->GetDataLayerInstances(ActorDescInstance->GetDataLayerInstanceNames().ToArray()));
							if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(OwningWorld))
							{
								UWorld* OuterWorld = ActorDescContainerInstance->GetTypedOuter<UWorld>();
								// Add parent container Data Layer Instances
								AActor* CurrentActor = OuterWorld ? Cast<AActor>(LevelInstanceSubsystem->GetOwningLevelInstance(OuterWorld->PersistentLevel)) : nullptr;
								while (CurrentActor)
								{
									DataLayerInstances.Append(bUseLevelContext ? CurrentActor->GetDataLayerInstancesForLevel() : CurrentActor->GetDataLayerInstances());
									CurrentActor = Cast<AActor>(LevelInstanceSubsystem->GetParentLevelInstance(CurrentActor));
								};
							}
							BuildDataLayers(DataLayerInstances, bUseLevelContext);
						}
					}
				}
			}
		};

		// List Actor's DataLayers part of the owning world, then those only part of the actor level
		BuildDataLayersWithContext(Item, false);
		BuildDataLayersWithContext(Item, true);

		return Builder.ToString();
	});

	FGetTextForItem ContentBundleInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item)->FString
	{	
		UContentBundleEngineSubsystem* ContentBundleEngineSubsystem = UContentBundleEngineSubsystem::Get();
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get(); Actor && Actor->GetContentBundleGuid().IsValid())
			{
				if (const UContentBundleDescriptor* Descriptor = ContentBundleEngineSubsystem->GetContentBundleDescriptor(Actor->GetContentBundleGuid()))
				{
					return Descriptor->GetDisplayName();
				}
			}
		}
		else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle; ActorDescInstance && ActorDescInstance->GetContentBundleGuid().IsValid())
			{
				if (const UContentBundleDescriptor* Descriptor = ContentBundleEngineSubsystem->GetContentBundleDescriptor(ActorDescInstance->GetContentBundleGuid()))
				{
					return Descriptor->GetDisplayName();
				}
			}
		}

		return TEXT("");
	});

	FGetTextForItem SubPackageInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				if (const ILevelInstanceInterface* ActorAsLevelInstance = Cast<ILevelInstanceInterface>(Actor))
				{
					return ActorAsLevelInstance->GetWorldAssetPackage();
				}
			}
		}
		else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				if (FName LevelPackage = ActorDescInstance->GetChildContainerPackage(); !LevelPackage.IsNone())
				{
					return ActorDescInstance->GetChildContainerPackage().ToString();
				}
			}
		}

		return FString();
	});

	FGetTextForItem SocketInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			AActor* Actor = ActorItem->Actor.Get();

			if (Actor)
			{
				return Actor->GetAttachParentSocketName().ToString();
			}
		}
		
		return FString();
	});

	FGetTextForItem InternalNameInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				return Actor->GetName();
			}
		}
		else if (const FComponentTreeItem* ComponentItem = Item.CastTo<FComponentTreeItem>())
		{
			if (UActorComponent* Component = ComponentItem->Component.Get())
			{
				return Component->GetName();
			}
		}
		else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				return ActorDescInstance->GetActorNameString();
			}
		}
		else if (const FActorFolderTreeItem* ActorFolderItem = Item.CastTo<FActorFolderTreeItem>())
		{
			if (const UActorFolder* ActorFolder = Cast<UActorFolder>(ActorFolderItem->GetActorFolder()))
			{
				return ActorFolder->GetName();
			}
		}

		return FString();
	});

	FGetTextForItem LevelInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				return FPackageName::GetShortName(Actor->GetPackage()->GetName());
			}
		}
		else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				return FPackageName::GetShortName(ActorDescInstance->GetActorPackage());
			}
		}
		else if (const FActorFolderTreeItem* ActorFolderItem = Item.CastTo<FActorFolderTreeItem>())
		{
			if (const UActorFolder* ActorFolder = Cast<UActorFolder>(ActorFolderItem->GetActorFolder()))
			{
				return FPackageName::GetShortName(ActorFolder->GetPackage()->GetName());
			}
		}

		return FString();
	});

	FGetTextForItem UncachedLightsInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			AActor* Actor = ActorItem->Actor.Get();
			if (Actor)
			{
				return FString::Printf(TEXT("%7d"), Actor->GetNumUncachedStaticLightingInteractions());
			}
		}
		return FString();
	});

	auto AddTextInfoColumn = [&InInitOptions](FName ColumnID, TAttribute<FText> ColumnName, FGetTextForItem ColumnInfo, const FText& ColumnTooltip = FText::GetEmpty())
	{
		InInitOptions.ColumnMap.Add(
			ColumnID,
			FSceneOutlinerColumnInfo(
				ESceneOutlinerColumnVisibility::Invisible,
				20,
				FCreateSceneOutlinerColumn::CreateStatic(
					&FTextInfoColumn::CreateTextInfoColumn,
					ColumnID,
					ColumnInfo,
					ColumnTooltip),
				true,
				TOptional<float>(),
				ColumnName));
	};

	// The "Level" column should be named "Package Short Name" in wp enabled levels
	FText LevelColumnName;
	if(WorldPtr && WorldPtr->PersistentLevel && WorldPtr->PersistentLevel->IsUsingExternalActors())
	{
		LevelColumnName = FSceneOutlinerBuiltInColumnTypes::PackageShortName_Localized();
	}
	else
	{
		LevelColumnName = FSceneOutlinerBuiltInColumnTypes::Level_Localized();
	}

	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::Mobility(), FSceneOutlinerBuiltInColumnTypes::Mobility_Localized(), MobilityInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::Level(), LevelColumnName, LevelInfoText);
	if (UWorldPartition* WorldPartition = WorldPtr ? WorldPtr->GetWorldPartition() : nullptr)
	{
		AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::DataLayer(), FSceneOutlinerBuiltInColumnTypes::DataLayer_Localized(), DataLayerInfoText);
		if (WorldPartition->IsContentBundleEnabled())
		{
			AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::ContentBundle(), FSceneOutlinerBuiltInColumnTypes::ContentBundle_Localized(), ContentBundleInfoText);
		}
	}
	else
	{
		AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::Layer(), FSceneOutlinerBuiltInColumnTypes::Layer_Localized(), LayerInfoText);
	}
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::SubPackage(), FSceneOutlinerBuiltInColumnTypes::SubPackage_Localized(), SubPackageInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::Socket(), FSceneOutlinerBuiltInColumnTypes::Socket_Localized(), SocketInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::IDName(), FSceneOutlinerBuiltInColumnTypes::IDName_Localized(), InternalNameInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::UncachedLights(), FSceneOutlinerBuiltInColumnTypes::UncachedLights_Localized(), UncachedLightsInfoText);
}

IMPLEMENT_MODULE(FSceneOutlinerModule, SceneOutliner);

#undef LOCTEXT_NAMESPACE //SceneOutlinerModule
