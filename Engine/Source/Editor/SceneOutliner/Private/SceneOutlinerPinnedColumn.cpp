// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerPinnedColumn.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Views/STreeView.h"
#include "IDocumentation.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerPinnedActorColumn"

bool FSceneOutlinerPinnedColumn::FSceneOutlinerPinnedStateCache::CheckChildren(const ISceneOutlinerTreeItem& Item) const
{
	if (const bool* const State = PinnedStateInfo.Find(&Item))
	{
		return *State;
	}

	bool bIsPinned = false;
	for (const auto& ChildPtr : Item.GetChildren())
	{
		FSceneOutlinerTreeItemPtr Child = ChildPtr.Pin();
		if (Child.IsValid() && GetPinnedState(*Child))
		{
			bIsPinned = true;
			break;
		}
	}
	PinnedStateInfo.Add(&Item, bIsPinned);
	
	return bIsPinned;
}

bool FSceneOutlinerPinnedColumn::FSceneOutlinerPinnedStateCache::GetPinnedState(const ISceneOutlinerTreeItem& Item) const
{
	if (Item.HasPinnedStateInfo())
	{
		if (const bool* const State = PinnedStateInfo.Find(&Item))
		{
			return *State;
		}

		const bool bIsPinned = Item.GetPinnedState();
		PinnedStateInfo.Add(&Item, bIsPinned);
		return bIsPinned;
	}

	return CheckChildren(Item);
}

void FSceneOutlinerPinnedColumn::FSceneOutlinerPinnedStateCache::Empty()
{
	PinnedStateInfo.Empty();
}

template<>
struct TWidgetTypeTraits<class SPinnedWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SPinnedWidget : public SImage
{
public:
	SLATE_BEGIN_ARGS(SPinnedWidget) {}
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, TWeakPtr<ISceneOutliner> InWeakOutliner, TWeakPtr<ISceneOutlinerTreeItem> InWeakTreeItem, const TWeakPtr<FSceneOutlinerPinnedColumn>& InWeakColumn, const STableRow<FSceneOutlinerTreeItemPtr>* InRow)
	{
		WeakTreeItem = InWeakTreeItem;
		WeakOutliner = InWeakOutliner;
		WeakColumn = InWeakColumn;
		Row = InRow;

		SImage::Construct(
			SImage::FArguments()
            .ColorAndOpacity(this, &SPinnedWidget::GetForegroundColor)
            .Image(this, &SPinnedWidget::GetBrush)
        );

		static const FName NAME_PinnedBrush = TEXT("Icons.Pinned");
		static const FName NAME_UnpinnedBrush = TEXT("Icons.Unpinned");

		PinnedBrush = FAppStyle::Get().GetBrush(NAME_PinnedBrush);
		UnpinnedBrush = FAppStyle::Get().GetBrush(NAME_UnpinnedBrush);
	}

private:
	bool IsPinned() const
	{
		return WeakTreeItem.IsValid() && WeakColumn.IsValid() ? WeakColumn.Pin()->IsItemPinned(*WeakTreeItem.Pin()) : false;
	}
	
	FReply HandleClick() const
	{
		if (!WeakTreeItem.IsValid() || !WeakOutliner.IsValid())
		{
			return FReply::Unhandled();
		}

		const auto Outliner = WeakOutliner.Pin();
		const auto TreeItem = WeakTreeItem.Pin();
		const auto& Tree = Outliner->GetTree();

		if (!IsPinned())
		{
			if (Tree.IsItemSelected(TreeItem.ToSharedRef()))
			{
				Outliner->PinSelectedItems();
			}
			else
			{
				Outliner->PinItems({ TreeItem });
			}
		}
		else
		{
			if (Tree.IsItemSelected(TreeItem.ToSharedRef()))
			{
				Outliner->UnpinSelectedItems();
			}
			else
			{
				Outliner->UnpinItems({ TreeItem });
			}
		}

		return FReply::Handled();
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		return HandleClick();
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			FReply Reply = HandleClick();
			return Reply.PreventThrottling();
		}
		
		return FReply::Unhandled();
	}

	const FSlateBrush* GetBrush() const
	{
		return IsPinned() ? PinnedBrush : UnpinnedBrush;
	}

	virtual FSlateColor GetForegroundColor() const override
	{
		const auto Outliner = WeakOutliner.Pin();
		const auto TreeItem = WeakTreeItem.Pin();
		const bool bIsSelected = Outliner->GetTree().IsItemSelected(TreeItem.ToSharedRef());
		
		if (!IsPinned())
		{
			if (!Row->IsHovered() && !bIsSelected)
			{
				return FLinearColor::Transparent;
			}
		}
		
		return IsHovered() ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground();
	}

	/** The tree item we relate to */
	TWeakPtr<ISceneOutlinerTreeItem> WeakTreeItem;

	/** Weak pointer back to the outliner */
	TWeakPtr<ISceneOutliner> WeakOutliner;
	/** Weak pointer back to the column to check cached pinned state */
	TWeakPtr<FSceneOutlinerPinnedColumn> WeakColumn;

	/** Weak pointer back to the row */
	const STableRow<FSceneOutlinerTreeItemPtr>* Row = nullptr;

	const FSlateBrush* PinnedBrush = nullptr;
	const FSlateBrush* UnpinnedBrush = nullptr;
};

FName FSceneOutlinerPinnedColumn::GetColumnID()
{
	return GetID();
}

SHeaderRow::FColumn::FArguments FSceneOutlinerPinnedColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("SceneOutlinerForceLoadColumnToolTip", "Force Load: Keep the selected items loaded in the editor even when they don't overlap a loaded World Partition region."), nullptr, "Shared/MenuEntries/SceneOutliner_ActorBrowsingMode", "ForceLoadTooltip"))
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(this, &FSceneOutlinerPinnedColumn::GetHeaderIcon)
		];
}

const TSharedRef<SWidget> FSceneOutlinerPinnedColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (TreeItem->ShouldShowPinnedState())
	{
		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SPinnedWidget, WeakSceneOutliner, TreeItem, SharedThis(this), &Row)
			.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("SceneOutlinerForceLoadWidgetTooltip", "Toggles Force Load: Keep the selected items loaded in the editor even when they don't overlap a loaded World Partition region."), nullptr, "Shared/MenuEntries/SceneOutliner_ActorBrowsingMode", "ForceLoadTooltip"))
		];
	}
	return SNullWidget::NullWidget;
}

void FSceneOutlinerPinnedColumn::Tick(double InCurrentTime, float InDeltaTime)
{
	PinnedStateCache.Empty();
}

bool FSceneOutlinerPinnedColumn::IsItemPinned(const ISceneOutlinerTreeItem& Item) const
{
	return PinnedStateCache.GetPinnedState(Item);
}

const FSlateBrush* FSceneOutlinerPinnedColumn::GetHeaderIcon() const
{
	return FAppStyle::Get().GetBrush("Icons.Unpinned");
}

#undef LOCTEXT_NAMESPACE