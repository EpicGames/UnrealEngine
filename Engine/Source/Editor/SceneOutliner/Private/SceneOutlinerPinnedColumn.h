// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerPublicTypes.h"
#include "ISceneOutlinerColumn.h"

template<typename ItemType> class STableRow;

/** A column for the SceneOutliner that displays the pin icon */
class FSceneOutlinerPinnedColumn : public ISceneOutlinerColumn
{
	struct FSceneOutlinerPinnedStateCache
	{
	public:
		/** Recursive over the children and check the pinned state */
		bool CheckChildren(const ISceneOutlinerTreeItem& Item) const;
		/** Get or cache the pinned state of an item */
		bool GetPinnedState(const ISceneOutlinerTreeItem& Item) const;

		/** Clear the cache */
		void Empty();
	private:
		/** Map of tree item to pinned state */
		mutable TMap<const ISceneOutlinerTreeItem*, bool> PinnedStateInfo;
	};
public:
	FSceneOutlinerPinnedColumn(ISceneOutliner& SceneOutliner) : WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared())) {}

	virtual ~FSceneOutlinerPinnedColumn() = default;

	static FName GetID() { return FSceneOutlinerBuiltInColumnTypes::Pinned(); }

	// Begin ISceneOutlinerColumn Implementation
	virtual FName GetColumnID() override;
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual void Tick(double InCurrentTime, float InDeltaTime) override;
	virtual bool SupportsSorting() const override { return false; }
	// End ISceneOutlinerColumn Implementation

	bool IsItemPinned(const ISceneOutlinerTreeItem& Item) const;
private:
	const FSlateBrush* GetHeaderIcon() const;
	TWeakPtr<ISceneOutliner> WeakSceneOutliner;

	/** Cached pinned state of items per-frame */
	FSceneOutlinerPinnedStateCache PinnedStateCache;
};
