// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerSCCHandler.h"
#include "ISourceControlModule.h"
#include "SourceControlOperations.h"
#include "Misc/MessageDialog.h"
#include "HAL/IConsoleManager.h"
#include "FileHelpers.h"
#include "SourceControlWindows.h"
#include "ISourceControlWindowsModule.h"
#include "UncontrolledChangelistsModule.h"
#include "AssetViewUtils.h"
#include "LevelEditor.h"
#include "RevisionControlStyle/RevisionControlStyle.h"

#define LOCTEXT_NAMESPACE "FSceneOutlinerSCCHandler"

FSceneOutlinerSCCHandler::FSceneOutlinerSCCHandler()
{
	if (FLevelEditorModule* LevelEditorModule = FModuleManager::LoadModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditorModule->OnMapChanged().AddRaw(this, &FSceneOutlinerSCCHandler::OnMapChanged);
	}
}

FSceneOutlinerSCCHandler::~FSceneOutlinerSCCHandler()
{
	if (FLevelEditorModule* LevelEditorModule = FModuleManager::LoadModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditorModule->OnMapChanged().RemoveAll(this);
	}
}

TSharedPtr<FSceneOutlinerTreeItemSCC> FSceneOutlinerSCCHandler::GetItemSourceControl(const FSceneOutlinerTreeItemPtr& InItem) const
{
	TSharedPtr<FSceneOutlinerTreeItemSCC>* Result = ItemSourceControls.Find(InItem);
	if (!Result)
	{
		Result = &ItemSourceControls.Add(InItem, MakeShared<FSceneOutlinerTreeItemSCC>(InItem));
		(*Result)->Initialize();
	}
	
	check(Result);
	return *Result;
}

bool FSceneOutlinerSCCHandler::AddSourceControlMenuOptions(UToolMenu* Menu, TArray<FSceneOutlinerTreeItemPtr> InSelectedItems)
{
	SelectedItems = InSelectedItems;

	CacheCanExecuteVars();

	if (ISourceControlModule::Get().IsEnabled() || FUncontrolledChangelistsModule::Get().IsEnabled())
	{
		FToolMenuSection& Section = Menu->AddSection("AssetContextSourceControl");

		if (bCanExecuteSCC)
		{
			// SCC sub menu
			Section.AddSubMenu(
				"SourceControlSubMenu",
				LOCTEXT("SourceControlSubMenuLabel", "Revision Control"),
				LOCTEXT("SourceControlSubMenuToolTip", "Revision control actions."),
				FNewToolMenuDelegate::CreateSP(this, &FSceneOutlinerSCCHandler::FillSourceControlSubMenu),
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateSP( this, &FSceneOutlinerSCCHandler::CanExecuteSourceControlActions )
					),
				EUserInterfaceActionType::Button,
				false,
				FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Icon", FRevisionControlStyleManager::GetStyleSetName() , "RevisionControl.Icon.ConnectedBadge")
				);
		}
		else
		{
			// SCC sub menu
			Section.AddMenuEntry(
				"SourceControlSubMenu",
				LOCTEXT("SourceControlSubMenuLabel", "Revision Control"),
				LOCTEXT("SourceControlSubMenuDisabledToolTip", "Disabled because one or more selected items are not external packages."),
				FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Icon"),
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateSP( this, &FSceneOutlinerSCCHandler::CanExecuteSCC )
				)
				);
		}

		return true;
	}

	return false;
}

bool FSceneOutlinerSCCHandler::AllowExecuteSourceControlRevert() const
{
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SourceControl.Revert.EnableFromSceneOutliner")))
	{
		return CVar->GetBool();
	}
	else
	{
		return false;
	}
}

bool FSceneOutlinerSCCHandler::AllowExecuteSourceControlRevertUnsaved() const
{
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SourceControl.RevertUnsaved.Enable")))
	{
		return CVar->GetBool();
	}
	else
	{
		return false;
	}
}

bool FSceneOutlinerSCCHandler::CanExecuteSourceControlActions() const
{
	return SelectedItems.Num() > 0;
}

void FSceneOutlinerSCCHandler::CacheCanExecuteVars()
{
	bCanExecuteSCC = true;
	bCanExecuteSCCCheckOut = false;
	bCanExecuteSCCCheckIn = false;
	bCanExecuteSCCRevert = false;
	bCanExecuteSCCHistory = false;
	bUsesSnapshots = false;
	bUsesChangelists = false;

	const bool bAllowRevert = AllowExecuteSourceControlRevert();
	const bool bAllowRevertUnsaved = AllowExecuteSourceControlRevertUnsaved();

	if ( ISourceControlModule::Get().IsEnabled() || FUncontrolledChangelistsModule::Get().IsEnabled())
	{
		for (FSceneOutlinerTreeItemPtr SelectedItem : SelectedItems)
		{
			TSharedPtr<FSceneOutlinerTreeItemSCC> SourceControl = GetItemSourceControl(SelectedItem);
			if (SourceControl == nullptr)
			{
				continue;
			}

			if (!SourceControl->HasValidPackage())
			{
				// this isn't an external package so we can't do anything with source control
				bCanExecuteSCC = false;

				// skip computing the rest of this package, but don't break
				// since getting a mixed external/internal selection is possible
				// and we change the UI state to disabled to show the user why
				// they aren't getting SCC actions
				continue;
			}

			// Check the SCC state for each package in the selected paths
			FSourceControlStatePtr SourceControlState = SourceControl->GetSourceControlState();
			if (SourceControlState.IsValid())
			{
				if ( SourceControlState->CanCheckout() )
				{
					bCanExecuteSCCCheckOut = true;
				}

				if( SourceControlState->IsSourceControlled() && !SourceControlState->IsAdded() )
				{
					bCanExecuteSCCHistory = true;
				}

				if ( SourceControlState->CanCheckIn() )
				{
					bCanExecuteSCCCheckIn = true;
				}

				if ( bAllowRevert )
				{
					if ( SourceControlState->CanRevert() )
					{
						bCanExecuteSCCRevert = true;
					}
					else if ( bAllowRevertUnsaved )
					{
						// If the package is dirty, allow a revert of the in-memory changes that have not yet been saved to disk.
						if (UPackage* Package = SourceControl->GetPackage())
						{
							if (Package->IsDirty())
							{
								bCanExecuteSCCRevert = true;
							}
						}
					}
				}
			}
		}

		if (ISourceControlModule::Get().GetProvider().IsAvailable())
		{
			ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();
			bUsesSnapshots = Provider.UsesSnapshots();
			bUsesChangelists = Provider.UsesChangelists();
		}
	}
}

bool FSceneOutlinerSCCHandler::CanExecuteSCC() const
{
	return bCanExecuteSCC;
}

bool FSceneOutlinerSCCHandler::CanExecuteSCCCheckOut() const
{
	return bCanExecuteSCCCheckOut;
}

bool FSceneOutlinerSCCHandler::CanExecuteSCCCheckIn() const
{
	return bCanExecuteSCCCheckIn && !bUsesSnapshots;
}

bool FSceneOutlinerSCCHandler::CanExecuteSCCRevert() const
{
	return bCanExecuteSCCRevert;
}

bool FSceneOutlinerSCCHandler::CanExecuteSCCHistory() const
{
	return bCanExecuteSCCHistory;
}

bool FSceneOutlinerSCCHandler::CanExecuteSCCRefresh() const
{
	return ISourceControlModule::Get().IsEnabled();
}

bool FSceneOutlinerSCCHandler::CanExecuteSCCShowInChangelist() const
{
	return bUsesChangelists;
}

void FSceneOutlinerSCCHandler::FillSourceControlSubMenu(UToolMenu* Menu)
{
	FToolMenuSection& Section = Menu->AddSection("AssetSourceControlActions", LOCTEXT("AssetSourceControlActionsMenuHeading", "Revision Control"));

	if ( CanExecuteSCCCheckOut() )
	{
		Section.AddMenuEntry(
			"SCCCheckOut",
			LOCTEXT("SCCCheckOut", "Check Out"),
			LOCTEXT("SCCCheckOutTooltip", "Checks out the selected asset from revision control."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.CheckOut"),
			FUIAction(
				FExecuteAction::CreateSP( this, &FSceneOutlinerSCCHandler::ExecuteSCCCheckOut ),
				FCanExecuteAction::CreateSP( this, &FSceneOutlinerSCCHandler::CanExecuteSCCCheckOut )
			)
		);
	}

	if ( CanExecuteSCCCheckIn() )
	{
		Section.AddMenuEntry(
			"SCCCheckIn",
			LOCTEXT("SCCCheckIn", "Check In"),
			LOCTEXT("SCCCheckInTooltip", "Checks in the selected asset to revision control."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Submit"),
			FUIAction(
				FExecuteAction::CreateSP( this, &FSceneOutlinerSCCHandler::ExecuteSCCCheckIn ),
				FCanExecuteAction::CreateSP( this, &FSceneOutlinerSCCHandler::CanExecuteSCCCheckIn )
			)
		);
	}

	Section.AddMenuEntry(
		"SCCRefresh",
		LOCTEXT("SCCRefresh", "Refresh"),
		LOCTEXT("SCCRefreshTooltip", "Updates the revision control status of the asset."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Refresh"),
		FUIAction(
			FExecuteAction::CreateSP( this, &FSceneOutlinerSCCHandler::ExecuteSCCRefresh ),
			FCanExecuteAction::CreateSP( this, &FSceneOutlinerSCCHandler::CanExecuteSCCRefresh )
			)
		);

	if( CanExecuteSCCHistory() )
	{
		Section.AddMenuEntry(
			"SCCHistory",
			LOCTEXT("SCCHistory", "History"),
			LOCTEXT("SCCHistoryTooltip", "Displays the revision control revision history of the selected asset."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.History"),
			FUIAction(
				FExecuteAction::CreateSP( this, &FSceneOutlinerSCCHandler::ExecuteSCCHistory ),
				FCanExecuteAction::CreateSP( this, &FSceneOutlinerSCCHandler::CanExecuteSCCHistory )
			)
		);
	}

	if (CanExecuteSCCRevert())
	{
		Section.AddMenuEntry(
			"SCCRevert",
			LOCTEXT("SCCRevert", "Revert"),
			LOCTEXT("SCCRevertTooltip", "Reverts the item to the state it was before it was checked out."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Revert"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FSceneOutlinerSCCHandler::ExecuteSCCRevert),
				FCanExecuteAction::CreateSP(this, &FSceneOutlinerSCCHandler::CanExecuteSCCRevert)
			)
		);
	}

	if (CanExecuteSCCShowInChangelist())
	{
		Section.AddSeparator(NAME_None);
		Section.AddMenuEntry(
			"SCCFindInChangelist",
			LOCTEXT("SCCShowInChangelist", "Show in Changelist"),
			LOCTEXT("SCCShowInChangelistTooltip", "Show the selected assets in the Changelist window."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.ChangelistsTab"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FSceneOutlinerSCCHandler::ExecuteSCCShowInChangelist),
				FCanExecuteAction::CreateSP(this, &FSceneOutlinerSCCHandler::CanExecuteSCCShowInChangelist)
			)
		);
	}
}

void FSceneOutlinerSCCHandler::GetSelectedPackageNames(TArray<FString>& OutPackageNames) const
{
	for (FSceneOutlinerTreeItemPtr SelectedItem : SelectedItems)
	{
		const TSharedPtr<FSceneOutlinerTreeItemSCC> SourceControl = GetItemSourceControl(SelectedItem);
		if (SourceControl == nullptr)
		{
			continue;
		}

		FString PackageName = SourceControl->GetPackageName();
		if (!PackageName.IsEmpty())
		{
			OutPackageNames.Add(PackageName);
		}
	}
}

void FSceneOutlinerSCCHandler::GetSelectedPackages(TArray<UPackage*>& OutPackages) const
{
	for (FSceneOutlinerTreeItemPtr SelectedItem : SelectedItems)
	{
		const TSharedPtr<FSceneOutlinerTreeItemSCC> SourceControl = GetItemSourceControl(SelectedItem);
		if (SourceControl == nullptr)
		{
			continue;
		}

		if (UPackage* Package = SourceControl->GetPackage())
		{
			OutPackages.Add(Package);
			continue;
		}
		if (UPackage* Package = SourceControl->LoadPackage())
		{
			OutPackages.Add(Package);
			continue;
		}
	}
}

void FSceneOutlinerSCCHandler::ExecuteSCCShowInChangelist()
{
	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);

	ISourceControlWindowsModule::Get().SelectFiles(SourceControlHelpers::PackageFilenames(PackageNames));
}

void FSceneOutlinerSCCHandler::ExecuteSCCRefresh()
{
	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);

	TSharedPtr<FUpdateStatus, ESPMode::ThreadSafe> Operation = ISourceControlOperation::Create<FUpdateStatus>();
	Operation->SetForceUpdate(true);

	ISourceControlModule::Get().GetProvider().Execute(Operation.ToSharedRef(), SourceControlHelpers::PackageFilenames(PackageNames), EConcurrency::Asynchronous);
}

void FSceneOutlinerSCCHandler::ExecuteSCCCheckOut()
{
	TArray<FString> PackagesToCheckOut;
	GetSelectedPackageNames(PackagesToCheckOut);

	if ( PackagesToCheckOut.Num() > 0 )
	{
		FEditorFileUtils::CheckoutPackages(PackagesToCheckOut, nullptr, /*bErrorIfAlreadyCheckedOut=*/false);
	}
}

void FSceneOutlinerSCCHandler::ExecuteSCCCheckIn()
{
	TArray<UPackage*> Packages;
	GetSelectedPackages(Packages);

	// Prompt the user to ask if they would like to first save any dirty packages they are trying to check-in
	const FEditorFileUtils::EPromptReturnCode UserResponse = FEditorFileUtils::PromptForCheckoutAndSave( Packages, true, true );

	// If the user elected to save dirty packages, but one or more of the packages failed to save properly OR if the user
	// canceled out of the prompt, don't follow through on the check-in process
	const bool bShouldProceed = ( UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Success || UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Declined );
	if ( bShouldProceed )
	{
		TArray<FString> PackageNames;
		GetSelectedPackageNames(PackageNames);

		const bool bUseSourceControlStateCache = true;
		const bool bCheckinGood = FSourceControlWindows::PromptForCheckin(bUseSourceControlStateCache, PackageNames);

		if (!bCheckinGood)
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "SCC_Checkin_Failed", "Check-in failed as a result of save failure."));
		}
	}
	else
	{
		// If a failure occurred, alert the user that the check-in was aborted. This warning shouldn't be necessary if the user cancelled
		// from the dialog, because they obviously intended to cancel the whole operation.
		if ( UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Failure )
		{
			FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "SCC_Checkin_Aborted", "Check-in aborted as a result of save failure.") );
		}
	}
}

void FSceneOutlinerSCCHandler::ExecuteSCCRevert()
{
	TArray<UPackage*> PackagesToSave;
	GetSelectedPackages(PackagesToSave);

	if (PackagesToSave.Num() > 0)
	{
		// To prevent a 'Save Dialog' from popping up during RevertAndReloadPackages we save the dirty packages here,
		// as the 'Save Dialog' is confusing during a revert operation, unlike the check in operation.
		UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, /*bOnlyDirty=*/true);
	}

	TArray<FString> PackagesToRevert;
	GetSelectedPackageNames(PackagesToRevert);

	if (PackagesToRevert.Num() > 0)
	{
		SourceControlHelpers::RevertAndReloadPackages(PackagesToRevert, /*bRevertAll=*/false, /*bReloadWorld=*/true);
	}
}

void FSceneOutlinerSCCHandler::ExecuteSCCHistory()
{
	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);

	FSourceControlWindows::DisplayRevisionHistory(PackageNames);
}

void FSceneOutlinerSCCHandler::OnMapChanged(UWorld* InWorld, EMapChangeType MapChangedType)
{
	if (MapChangedType == EMapChangeType::NewMap)
	{
		ItemSourceControls.Empty();
	}
}

#undef LOCTEXT_NAMESPACE
