// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "SceneOutlinerPublicTypes.h"

#include "SceneOutlinerModule.h"

void FSharedSceneOutlinerData::UseDefaultColumns()
{
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	// Create an instance of every default column type
	for (auto& DefaultColumn : SceneOutlinerModule.DefaultColumnMap)
	{
		ColumnMap.Add(DefaultColumn.Key, DefaultColumn.Value);
	}
}
