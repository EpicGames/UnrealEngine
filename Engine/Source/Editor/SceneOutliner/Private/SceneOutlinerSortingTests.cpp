// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AutomationTest.h"
#include "SortHelper.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSortBasicTest, "System.Editor.Scene Outliner.SortBasic", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FSortBasicTest::RunTest(const FString& Parameters)
{
	TArray<SceneOutliner::FNumericStringWrapper> Muddled;

	Muddled.Emplace(TEXT("String 23"));
	Muddled.Emplace(TEXT("String 230"));
	Muddled.Emplace(TEXT("String 110"));
	Muddled.Emplace(TEXT("100"));
	Muddled.Emplace(TEXT("001"));
	Muddled.Emplace(TEXT("1"));
	Muddled.Emplace(TEXT("String 1"));
	Muddled.Emplace(TEXT("String 10"));
	Muddled.Emplace(TEXT("a 1 b"));
	Muddled.Emplace(TEXT("a 20 c"));
	Muddled.Emplace(TEXT("a 001 b 6"));
	Muddled.Emplace(TEXT("a 001 b 5"));
	Muddled.Emplace(TEXT("a 10 b"));
	Muddled.Emplace(TEXT("abc 123"));
	Muddled.Emplace(TEXT("xyz 456"));


	TArray<SceneOutliner::FNumericStringWrapper> Sorted;

	Sorted.Emplace(TEXT("1"));
	Sorted.Emplace(TEXT("001"));
	Sorted.Emplace(TEXT("100"));

	Sorted.Emplace(TEXT("a 1 b"));
	Sorted.Emplace(TEXT("a 001 b 5"));
	Sorted.Emplace(TEXT("a 001 b 6"));
	Sorted.Emplace(TEXT("a 10 b"));
	Sorted.Emplace(TEXT("a 20 c"));

	Sorted.Emplace(TEXT("abc 123"));
	
	Sorted.Emplace(TEXT("String 1"));
	Sorted.Emplace(TEXT("String 10"));
	Sorted.Emplace(TEXT("String 23"));
	Sorted.Emplace(TEXT("String 110"));
	Sorted.Emplace(TEXT("String 230"));

	Sorted.Emplace(TEXT("xyz 456"));

	Muddled.Sort([](const SceneOutliner::FNumericStringWrapper& A, const SceneOutliner::FNumericStringWrapper& B){
		return A < B;
	});

	return Sorted == Muddled;
}
