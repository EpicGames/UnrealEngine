// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerSourceControlColumn.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Views/STreeView.h"
#include "ActorTreeItem.h"
#include "ActorDescTreeItem.h"
#include "ActorFolderTreeItem.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "ISourceControlModule.h"
#include "SceneOutlinerTreeItemSCC.h"
#include "SSourceControlWidget.h"
#include "Misc/MessageDialog.h"
#include "RevisionControlStyle/RevisionControlStyle.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerSourceControlColumn"

FName FSceneOutlinerSourceControlColumn::GetColumnID()
{
	return GetID();
}

SHeaderRow::FColumn::FArguments FSceneOutlinerSourceControlColumn::ConstructHeaderRowColumn()
{
	TSharedRef<SLayeredImage> HeaderRowIcon = SNew(SLayeredImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon"));
	
	HeaderRowIcon->AddLayer(TAttribute<const FSlateBrush*>::CreateSP(this, &FSceneOutlinerSourceControlColumn::GetHeaderIconBadge));
	
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(GetDisplayName())
		[
			HeaderRowIcon
		];
}

const TSharedRef<SWidget> FSceneOutlinerSourceControlColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	TSharedPtr<FSceneOutlinerTreeItemSCC> SourceControl = WeakSceneOutliner.Pin()->GetItemSourceControl(TreeItem);
	if (SourceControl.IsValid() && SourceControl->HasValidPackage())
	{
		TSharedRef<SSourceControlWidget> Widget = SNew(SSourceControlWidget, SourceControl);
		
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				Widget
			];
	}
	return SNullWidget::NullWidget;
}

const FSlateBrush* FSceneOutlinerSourceControlColumn::GetHeaderIconBadge() const
{
	if (ISourceControlModule::Get().IsEnabled())
	{
		return FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon.ConnectedBadge");
	}
	else
	{
		return nullptr;
	}
}

#undef LOCTEXT_NAMESPACE