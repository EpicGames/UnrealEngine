// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerStandaloneTypes.h"

#include "ActorTreeItem.h"
#include "EditorActorFolders.h"
#include "Framework/Application/SlateApplication.h"
#include "ISceneOutlinerTreeItem.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"


#define LOCTEXT_NAMESPACE "SceneOutlinerStandaloneTypes"

uint32 FSceneOutlinerTreeItemType::NextUniqueID = 0;
const FSceneOutlinerTreeItemType ISceneOutlinerTreeItem::Type;

const FLinearColor FSceneOutlinerCommonLabelData::DarkColor(0.15f, 0.15f, 0.15f);

TOptional<FLinearColor> FSceneOutlinerCommonLabelData::GetForegroundColor(TWeakPtr<ISceneOutliner> WeakSceneOutliner, const ISceneOutlinerTreeItem& TreeItem)
{
	if (!TreeItem.IsValid())
	{
		return DarkColor;
	}

	// Darken items that aren't suitable targets for an active drag and drop action
	if (FSlateApplication::Get().IsDragDropping())
	{
		TSharedPtr<FDragDropOperation> DragDropOp = FSlateApplication::Get().GetDragDroppingContent();

		FSceneOutlinerDragDropPayload DraggedObjects(*DragDropOp);

		if (const TSharedPtr<ISceneOutliner> Outliner = WeakSceneOutliner.Pin())
		{
			if (Outliner->GetMode()->ParseDragDrop(DraggedObjects, *DragDropOp) && !Outliner->GetMode()->ValidateDrop(TreeItem, DraggedObjects).IsValid())
			{
				return DarkColor;
			}
		}
		else
		{
			return DarkColor;
		}
	}

	if (!TreeItem.CanInteract())
	{
		return DarkColor;
	}

	if(const FActorTreeItem* ActorTreeItem = TreeItem.CastTo<FActorTreeItem>())
	{
		AActor* Actor = ActorTreeItem->Actor.Get();
		
		if (!Actor)
		{
			// Deleted actor!
			return FLinearColor(0.2f, 0.2f, 0.25f);
		}

		UWorld* OwningWorld = Actor->GetWorld();
		if (!OwningWorld)
		{
			// Deleted world!
			return FLinearColor(0.2f, 0.2f, 0.25f);
		}

		const bool bRepresentingPIEWorld = Actor->GetWorld()->IsPlayInEditor();
		if (bRepresentingPIEWorld && !ActorTreeItem->bExistsInCurrentWorldAndPIE)
		{
			// Highlight actors that are exclusive to PlayWorld
			return FLinearColor(0.9f, 0.8f, 0.4f);
		}
	}
	
	return TOptional<FLinearColor>();
}

TOptional<FLinearColor> FSceneOutlinerCommonLabelData::GetForegroundColor(const ISceneOutlinerTreeItem& TreeItem) const
{
	return GetForegroundColor(WeakSceneOutliner, TreeItem);
}

bool FSceneOutlinerCommonLabelData::CanExecuteRenameRequest(const ISceneOutlinerTreeItem& Item) const
{
	if (const ISceneOutliner* SceneOutliner = WeakSceneOutliner.Pin().Get())
	{
		return SceneOutliner->CanExecuteRenameRequest(Item);
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
