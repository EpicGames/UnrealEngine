// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerTextInfoColumn.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerPublicTypes.h"
#include "SortHelper.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerTextInfoColumn"


TSharedRef<ISceneOutlinerColumn> FTextInfoColumn::CreateTextInfoColumn(ISceneOutliner& Outliner, const FName InColumnName, const FGetTextForItem InGetTextForItem, const FText InColumnToolTip)
{
	return TSharedRef<ISceneOutlinerColumn>(MakeShareable(new FTextInfoColumn(Outliner, InColumnName, InGetTextForItem, InColumnToolTip)));
}

FTextInfoColumn::FTextInfoColumn(ISceneOutliner& Outliner, const FName InColumnName, const FGetTextForItem& InGetTextForItem, const FText InColumnToolTip):
	ColumnName(InColumnName),
	ColumnToolTip(InColumnToolTip),
	SceneOutlinerWeak(StaticCastSharedRef<ISceneOutliner>(Outliner.AsShared())),
	GetTextForItem(InGetTextForItem)
{

}

FName FTextInfoColumn::GetColumnID()
{
	return ColumnName;
}

SHeaderRow::FColumn::FArguments FTextInfoColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(ColumnName)
		.DefaultTooltip(!ColumnToolTip.IsEmptyOrWhitespace() ? ColumnToolTip : TAttribute<FText>())
		.FillWidth(2)
		.HeaderComboVisibility(EHeaderComboVisibility::OnHover);
}

const TSharedRef< SWidget > FTextInfoColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	auto SceneOutliner = SceneOutlinerWeak.Pin();
	check(SceneOutliner.IsValid());

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	TSharedRef<STextBlock> MainText = SNew(STextBlock)
		.Text(this, &FTextInfoColumn::GetInfoForItem, TWeakPtr<ISceneOutlinerTreeItem>(TreeItem))
		.HighlightText(SceneOutliner->GetFilterHighlightText())
		.ColorAndOpacity(FSlateColor::UseSubduedForeground());

	HorizontalBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(8, 0, 0, 0)
		[
			MainText
		];
	
	return HorizontalBox;
}

void FTextInfoColumn::PopulateSearchStrings(const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings) const
{
	FString String = GetTextForItem.Execute(Item);
	if (String.Len())
	{
		OutSearchStrings.Add(MoveTemp(String));
	}
}

bool FTextInfoColumn::SupportsSorting() const
{
	return true; // Text Columns are always sortable
}

void FTextInfoColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const
{
	FSceneOutlinerSortHelper<FString>()
		.Primary([this](const ISceneOutlinerTreeItem& Item) { return GetTextForItem.Execute(Item); }, SortMode)
		.Sort(RootItems);
}

FText FTextInfoColumn::GetInfoForItem(TWeakPtr<ISceneOutlinerTreeItem> TreeItem) const
{
	auto Item = TreeItem.Pin();
	return Item.IsValid() ? FText::FromString(GetTextForItem.Execute(*Item)) : FText::GetEmpty();
}


#undef LOCTEXT_NAMESPACE
