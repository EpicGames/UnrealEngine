// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerTreeItemSCC.h"

#include "ActorTreeItem.h"
#include "ActorDescTreeItem.h"
#include "ActorFolderTreeItem.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "PackageTools.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "SceneOutlinerHelpers.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "UObject/Package.h"
#include "UncontrolledChangelistsModule.h"

FSceneOutlinerTreeItemSCC::FSceneOutlinerTreeItemSCC(FSceneOutlinerTreeItemPtr InTreeItemPtr)
{
	TreeItemPtr = InTreeItemPtr;
}

FSceneOutlinerTreeItemSCC::~FSceneOutlinerTreeItemSCC()
{
	if(FUncontrolledChangelistsModule::IsAvailable())
	{
		FUncontrolledChangelistsModule& UncontrolledChangelistModule = FUncontrolledChangelistsModule::Get();
		UncontrolledChangelistModule.OnUncontrolledChangelistModuleChanged.Remove(UncontrolledChangelistChangedHandle);
	}
	
	DisconnectSourceControl();
}

void FSceneOutlinerTreeItemSCC::Initialize()
{
	if (TreeItemPtr.IsValid())
	{
		ExternalPackageName = TreeItemPtr->GetPackageName();
		ExternalPackageFileName = !ExternalPackageName.IsEmpty() ? USourceControlHelpers::PackageFilename(ExternalPackageName) : FString();
		
		if (FActorTreeItem* ActorItem = TreeItemPtr->CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				ActorPackingModeChangedDelegateHandle = Actor->OnPackagingModeChanged.AddLambda([this](AActor* InActor, bool bExternal)
				{
					if (bExternal)
					{
						ExternalPackageName = InActor->GetExternalPackage()->GetName();
						ExternalPackageFileName = USourceControlHelpers::PackageFilename(ExternalPackageName);
						ConnectSourceControl();
					}
					else
					{
						ExternalPackageName = FString();
						ExternalPackageFileName = FString();
						DisconnectSourceControl();
					}
				});
			}
		}
		
		if (!ExternalPackageFileName.IsEmpty())
		{
			ConnectSourceControl();
		}
	}

	FUncontrolledChangelistsModule& UncontrolledChangelistModule = FUncontrolledChangelistsModule::Get();

	UncontrolledChangelistChangedHandle = UncontrolledChangelistModule.OnUncontrolledChangelistModuleChanged.AddLambda([this, WeakThis = AsWeak()]()
	{
		if (WeakThis.IsValid())
		{
			HandleUncontrolledChangelistsStateChanged();
		}
	});

	// Call the delegate to update the initial uncontrolled state
	HandleUncontrolledChangelistsStateChanged();
}

FSourceControlStatePtr FSceneOutlinerTreeItemSCC::GetSourceControlState()
{
	return ISourceControlModule::Get().GetProvider().GetState(ExternalPackageFileName, EStateCacheUsage::Use);
}

FSourceControlStatePtr FSceneOutlinerTreeItemSCC::RefreshSourceControlState()
{
	return ISourceControlModule::Get().GetProvider().GetState(ExternalPackageFileName, EStateCacheUsage::ForceUpdate);
}

UPackage* FSceneOutlinerTreeItemSCC::GetPackage() const
{
	return FindPackage(nullptr, *ExternalPackageName);
}

UPackage* FSceneOutlinerTreeItemSCC::LoadPackage() const
{
	return UPackageTools::LoadPackage(ExternalPackageName);
}

void FSceneOutlinerTreeItemSCC::ConnectSourceControl()
{
	check(!ExternalPackageFileName.IsEmpty());

	ISourceControlModule& SCCModule = ISourceControlModule::Get();

	SourceControlProviderChangedDelegateHandle = SCCModule.RegisterProviderChanged(FSourceControlProviderChanged::FDelegate::CreateLambda([this, WeakThis = AsWeak()](ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
	{
		if (WeakThis.IsValid())
		{
			HandleSourceControlProviderChanged(OldProvider, NewProvider);
		}
	}));
	
	SourceControlStateChangedDelegateHandle = SCCModule.GetProvider().RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateLambda([this, WeakThis = AsWeak()]()
	{
		if (WeakThis.IsValid())
		{
			HandleSourceControlStateChanged(EStateCacheUsage::Use);
		}
	}));
	
	// Check if there is already a cached state for this item
	FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(ExternalPackageFileName, EStateCacheUsage::Use);
	if (SourceControlState.IsValid() && !SourceControlState->IsUnknown())
	{
		BroadcastNewState(SourceControlState);
	}
	else if (UPackage* Package = GetPackage())
	{
		// There's nothing to query if the package didn't come from disk or hasn't yet been saved to disk
		if (Package->GetHasBeenEndLoaded())
		{
			SCCModule.QueueStatusUpdate(ExternalPackageFileName);
		}
	}
	else
	{
		SCCModule.QueueStatusUpdate(ExternalPackageFileName);
	}
}

void FSceneOutlinerTreeItemSCC::DisconnectSourceControl()
{
	if (TreeItemPtr.IsValid())
	{
		if (FActorTreeItem* ActorItem = TreeItemPtr->CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				Actor->OnPackagingModeChanged.Remove(ActorPackingModeChangedDelegateHandle);
			}
		}
	}
	ISourceControlModule::Get().GetProvider().UnregisterSourceControlStateChanged_Handle(SourceControlStateChangedDelegateHandle);
	ISourceControlModule::Get().UnregisterProviderChanged(SourceControlProviderChangedDelegateHandle);
}

void FSceneOutlinerTreeItemSCC::HandleSourceControlStateChanged(EStateCacheUsage::Type CacheUsage)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSceneOutlinerTreeItemSCC::HandleSourceControlStateChanged);

	FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(ExternalPackageFileName, CacheUsage);
	if (SourceControlState.IsValid())
	{
		BroadcastNewState(SourceControlState);
	}
}

void FSceneOutlinerTreeItemSCC::HandleSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	OldProvider.UnregisterSourceControlStateChanged_Handle(SourceControlStateChangedDelegateHandle);
	
	/* Early exit if the engine is shutting down, in case there are any lingering SCC items in the Outliner that haven't
	 * been destroyed yet calling into modules that have been unloaded
	 */
	if (IsEngineExitRequested())
	{
		return;
	}

	SourceControlStateChangedDelegateHandle = NewProvider.RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateLambda([this, WeakThis = AsWeak()]()
	{
		if (WeakThis.IsValid())
		{
			HandleSourceControlStateChanged(EStateCacheUsage::Use);
		}
	}));
	
	BroadcastNewState(nullptr);

	ISourceControlModule::Get().QueueStatusUpdate(ExternalPackageFileName);
}

void FSceneOutlinerTreeItemSCC::BroadcastNewState(FSourceControlStatePtr SourceControlState)
{
	OnSourceControlStateChanged.ExecuteIfBound(SourceControlState);
}

void FSceneOutlinerTreeItemSCC::HandleUncontrolledChangelistsStateChanged()
{
	TSharedPtr<FUncontrolledChangelistState> PrevUncontrolledChangelistState = UncontrolledChangelistState;
	
	UncontrolledChangelistState = nullptr;
	
	TArray<FUncontrolledChangelistStateRef> UncontrolledChangelistStates = FUncontrolledChangelistsModule::Get().GetChangelistStates();

	for (const TSharedRef<FUncontrolledChangelistState>& UncontrolledChangelistStateRef : UncontrolledChangelistStates)
	{
		if (UncontrolledChangelistStateRef->ContainsFilename(ExternalPackageFileName))
		{
			UncontrolledChangelistState = UncontrolledChangelistStateRef;
			break;
		}
	}

	// Broadcast the delegate if our uncontrolled status was changed
	if (UncontrolledChangelistState != PrevUncontrolledChangelistState)
	{
		OnUncontrolledChangelistsStateChanged.ExecuteIfBound(UncontrolledChangelistState);
	}
}