// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneOutlinerUnsavedColumn.h"
#include "ActorTreeItem.h"
#include "SourceControlHelpers.h"
#include "UnsavedAssetsTrackerModule.h"
#include "SortHelper.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"
#include "SceneOutlinerHelpers.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerActorUnsavedColumn"

template<>
struct TWidgetTypeTraits<class SUnsavedActorWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SUnsavedActorWidget : public SImage
{
public:
	SLATE_BEGIN_ARGS(SUnsavedActorWidget) {}
	SLATE_END_ARGS()

	~SUnsavedActorWidget();

	/** Construct this widget */
	void Construct(const FArguments& InArgs, TWeakPtr<ISceneOutlinerTreeItem> InWeakTreeItem);
	
	bool IsUnsaved() const;
	
	void OnUnsavedAssetAdded(const FString& FileAbsPathname);
	void OnUnsavedAssetRemoved(const FString& FileAbsPathname);

private:
	void UpdateImage();
	void UpdateExternalPackageFilename();
	
private:
	FString ExternalPackageFilename;
	TWeakPtr<ISceneOutlinerTreeItem> WeakTreeItem;
	bool bIsUnsaved;
	bool bIsSavable;

	FDelegateHandle OnPackagingModeChangedHandle;
	FDelegateHandle OnUnsavedAssetAddedHandle;
	FDelegateHandle OnUnsavedAssetRemovedHandle;
};

void SUnsavedActorWidget::Construct(const FArguments& InArgs, TWeakPtr<ISceneOutlinerTreeItem> InWeakTreeItem)
{
	WeakTreeItem = InWeakTreeItem;

	SImage::Construct(
			SImage::FArguments()
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FStyleDefaults::GetNoBrush()));

	UpdateExternalPackageFilename();

	// Handle Actor package change
	if (FActorTreeItem* ActorItem = WeakTreeItem.Pin()->CastTo<FActorTreeItem>())
	{
		if (AActor* Actor = ActorItem->Actor.Get())
		{
			OnPackagingModeChangedHandle = Actor->OnPackagingModeChanged.AddLambda([WeakThis = AsWeak()](AActor* InActor, bool bExternal)
			{
				if (TSharedPtr<SUnsavedActorWidget> This = StaticCastSharedPtr<SUnsavedActorWidget>(WeakThis.Pin()))
				{
					This->UpdateExternalPackageFilename();
				}
			});
		}
	}
}

SUnsavedActorWidget::~SUnsavedActorWidget()
{
	if (FActorTreeItem* ActorItem = WeakTreeItem.IsValid() ? WeakTreeItem.Pin()->CastTo<FActorTreeItem>() : nullptr)
	{
		if (AActor* Actor = ActorItem->Actor.Get())
		{
			Actor->OnPackagingModeChanged.Remove(OnPackagingModeChangedHandle);
		}
	}

	if (FUnsavedAssetsTrackerModule* UnsavedAssetsTrackerModule = FModuleManager::GetModulePtr<FUnsavedAssetsTrackerModule>("UnsavedAssetsTracker"))
	{
		UnsavedAssetsTrackerModule->OnUnsavedAssetAdded.Remove(OnUnsavedAssetAddedHandle);
		UnsavedAssetsTrackerModule->OnUnsavedAssetRemoved.Remove(OnUnsavedAssetRemovedHandle);
	}
}

void SUnsavedActorWidget::UpdateExternalPackageFilename()
{
	const bool bWasExternal = !ExternalPackageFilename.IsEmpty();
		
	const FString ExternalPackageName = WeakTreeItem.IsValid() ? WeakTreeItem.Pin()->GetPackageName() : FString();
	ExternalPackageFilename = !ExternalPackageName.IsEmpty() ? USourceControlHelpers::PackageFilename(ExternalPackageName) : FString();

	FUnsavedAssetsTrackerModule& UnsavedAssetsTrackerModule = FUnsavedAssetsTrackerModule::Get();

	// Register/Unregister if needed
	if (bWasExternal && ExternalPackageFilename.IsEmpty())
	{
		UnsavedAssetsTrackerModule.OnUnsavedAssetAdded.Remove(OnUnsavedAssetAddedHandle);
		OnUnsavedAssetAddedHandle.Reset();

		UnsavedAssetsTrackerModule.OnUnsavedAssetRemoved.Remove(OnUnsavedAssetRemovedHandle);
		OnUnsavedAssetRemovedHandle.Reset();
	}
	else if (!bWasExternal && !ExternalPackageFilename.IsEmpty())
	{
		OnUnsavedAssetAddedHandle = UnsavedAssetsTrackerModule.OnUnsavedAssetAdded.AddSP(this, &SUnsavedActorWidget::OnUnsavedAssetAdded);
		OnUnsavedAssetRemovedHandle = UnsavedAssetsTrackerModule.OnUnsavedAssetRemoved.AddSP(this, &SUnsavedActorWidget::OnUnsavedAssetRemoved);
	}

	bIsUnsaved = UnsavedAssetsTrackerModule.IsAssetUnsaved(ExternalPackageFilename);
	bIsSavable = !ExternalPackageFilename.IsEmpty();

	UpdateImage();
}

bool SUnsavedActorWidget::IsUnsaved() const
{
	return bIsUnsaved;
}

void SUnsavedActorWidget::OnUnsavedAssetAdded(const FString& FileAbsPathname)
{
	if (FileAbsPathname == ExternalPackageFilename)
	{
		// We should never be desynced, i.e if this item was added as an unsaved asset bIsUnsavedAsset MUST be false before
		check(!bIsUnsaved)
		bIsUnsaved = true;
		
		UpdateImage();
	}
}

void SUnsavedActorWidget::OnUnsavedAssetRemoved(const FString& FileAbsPathname)
{
	if (FileAbsPathname == ExternalPackageFilename)
	{
		// We should never be desynced, i.e if this item was removed from the unsaved asset list bIsUnsavedAsset MUST be true before
		check(bIsUnsaved)
		bIsUnsaved = false;

		UpdateImage();
	}
}

void SUnsavedActorWidget::UpdateImage()
{
	if (bIsUnsaved)
	{
		SetImage(FAppStyle::GetBrush("Icons.DirtyBadge"));
	}
	else if (bIsSavable)
	{
		SetImage(FAppStyle::GetBrush("Icons.SaveableBadge"));
	}
	else
	{
		SetImage(nullptr);
	}
}


FName FSceneOutlinerActorUnsavedColumn::GetColumnID()
{
	return GetID();
}

SHeaderRow::FColumn::FArguments FSceneOutlinerActorUnsavedColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(GetDisplayName())
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.DirtyBadge"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
}

const TSharedRef<SWidget> FSceneOutlinerActorUnsavedColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{	
	return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SUnsavedActorWidget, TreeItem.ToWeakPtr())
			];
}

void FSceneOutlinerActorUnsavedColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const
{
	FUnsavedAssetsTrackerModule& UnsavedAssetsTrackerModule = FUnsavedAssetsTrackerModule::Get();
	FSceneOutlinerSortHelper<bool, SceneOutliner::FNumericStringWrapper>()
		/** Sort by unsaved first */
		.Primary([UnsavedAssetsTrackerModule](const ISceneOutlinerTreeItem& Item)
		{
			if (const FString PackageName = Item.GetPackageName(); !PackageName.IsEmpty())
			{
				return !UnsavedAssetsTrackerModule.IsAssetUnsaved(USourceControlHelpers::PackageFilename(PackageName));
			}
			return true;
		}, SortMode)
		/** Then by type */
		.Secondary([this](const ISceneOutlinerTreeItem& Item){ return SceneOutliner::FNumericStringWrapper(Item.GetDisplayString()); }, SortMode)
		.Sort(RootItems);
}

#undef LOCTEXT_NAMESPACE