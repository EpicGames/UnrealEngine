// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerPublicTypes.h"
#include "ISceneOutlinerColumn.h"

template<typename ItemType> class STableRow;
class SUnsavedActorWidget;

/** A column for the SceneOutliner that displays if the actor is unsaved */
class FSceneOutlinerActorUnsavedColumn : public ISceneOutlinerColumn
{
public:
	FSceneOutlinerActorUnsavedColumn(ISceneOutliner& SceneOutliner) : WeakOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared())) {}
	virtual ~FSceneOutlinerActorUnsavedColumn() = default;

	static FName GetID() { return FSceneOutlinerBuiltInColumnTypes::Unsaved(); }
	static FText GetDisplayName() { return FSceneOutlinerBuiltInColumnTypes::Unsaved_Localized(); }

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation
	virtual FName GetColumnID() override;
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return true; }
	virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const override;
	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////

private:
	TWeakPtr<ISceneOutliner> WeakOutliner;
};
