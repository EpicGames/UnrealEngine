// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldTreeItem.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/SToolTip.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "SceneOutlinerDragDrop.h"
#include "SSceneOutliner.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenus.h"
#include "LevelEditor.h"
#include "EditorActorFolders.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"
#include "IDocumentation.h"
#include "FolderTreeItem.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "SceneOutliner_WorldTreeItem"

namespace SceneOutliner
{
	FText GetWorldDescription(UWorld* World)
	{
		FText Description;
		if (World)
		{
			FText PostFix;
			const FWorldContext* WorldContext = nullptr;
			for (const FWorldContext& Context : GEngine->GetWorldContexts())
			{
				if (Context.World() == World)
				{
					WorldContext = &Context;
					break;
				}
			}

			if (World->WorldType == EWorldType::PIE)
			{
				switch (World->GetNetMode())
				{
				case NM_Client:
					if (WorldContext)
					{
						PostFix = FText::Format(LOCTEXT("ClientPostfixFormat", "(Client {0})"), FText::AsNumber(WorldContext->PIEInstance - 1));
					}
					else
					{
						PostFix = LOCTEXT("ClientPostfix", "(Client)");
					}
					break;
				case NM_DedicatedServer:
				case NM_ListenServer:
					PostFix = LOCTEXT("ServerPostfix", "(Server)");
					break;
				case NM_Standalone:
					PostFix = LOCTEXT("PlayInEditorPostfix", "(Play In Editor)");
					break;
				}
			}
			else if (World->WorldType == EWorldType::Editor)
			{
				PostFix = LOCTEXT("EditorPostfix", "(Editor)");
			}

			Description = FText::Format(LOCTEXT("WorldFormat", "{0} {1}"), FText::FromString(World->GetFName().GetPlainNameString()), PostFix);
		}

		return Description;
	}
}

const FSceneOutlinerTreeItemType FWorldTreeItem::Type(&ISceneOutlinerTreeItem::Type);
TMap<FName, FWorldTreeItem::FContextMenuProviderFunc> FWorldTreeItem::ContextMenuProviders;

template<>
struct TWidgetTypeTraits<struct SWorldTreeLabel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

struct SWorldTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SWorldTreeLabel) {}
	SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, FWorldTreeItem& WorldItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		TreeItemPtr = StaticCastSharedRef<FWorldTreeItem>(WorldItem.AsShared());
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
			[
				SNew(SBox)
				.WidthOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
				.HeightOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
				[
					SNew(SImage)
					.Image(FSlateIconFinder::FindIconBrushForClass(UWorld::StaticClass()))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.ToolTipText(LOCTEXT("WorldIcon_Tooltip", "World"))
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 2.0f)
			[
				SNew(STextBlock)
				.Text(this, &SWorldTreeLabel::GetDisplayText)
				.HighlightText(SceneOutliner.GetFilterHighlightText())
				.ColorAndOpacity(this, &SWorldTreeLabel::GetForegroundColorInternal)
				.OnDoubleClicked(this, &SWorldTreeLabel::HandleDoubleClick)
				.ToolTip(IDocumentation::Get()->CreateToolTip(
					TAttribute<FText>(this, &SWorldTreeLabel::GetTooltipText),
					nullptr,
					TEXT("Shared/LevelEditor/SceneOutliner"),
					TEXT("WorldSettingsLabel")
				))
			]
		];
		SetForegroundColor(TAttribute<FSlateColor>::CreateRaw(this, &SWorldTreeLabel::GetForegroundColorInternal));
	}

private:
	TWeakPtr<FWorldTreeItem> TreeItemPtr;

	FText GetDisplayText() const
	{
		auto Item = TreeItemPtr.Pin();
		return Item.IsValid() ? FText::FromString(Item->GetDisplayString()) : FText();
	}

	FText GetTooltipText() const
	{
		auto Item = TreeItemPtr.Pin();
		FText PersistentLevelDisplayName = Item.IsValid() ? FText::FromString(Item->GetWorldName()) : FText();
		if (Item.IsValid() && Item->CanInteract())
		{
			return FText::Format(LOCTEXT("WorldLabel_Tooltip", "The {0} level. Double-click to edit World Settings."), PersistentLevelDisplayName);
		}
		else
		{
			return FText::Format(LOCTEXT("WorldLabel_TooltipNonInteractive", "The {0} level."), PersistentLevelDisplayName);
		}
	}

	/**
	 * Returns the Foreground Color of this widget.
	 * Used to avoid overriding SCompoundWidget::GetForegroundColor for invalidation to work properly.
	 */
	FSlateColor GetForegroundColorInternal() const
	{
		if (auto BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItemPtr.Pin()))
		{
			return BaseColor.GetValue();
		}

		return FSlateColor::UseForeground();
	}

	FReply HandleDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent) const
	{
		auto Item = TreeItemPtr.Pin();
		if (Item.IsValid() && Item->CanInteract())
		{
			Item->OpenWorldSettings();
		}
		return FReply::Handled();
	}
};

FWorldTreeItem::FWorldTreeItem(UWorld* InWorld)
	: ISceneOutlinerTreeItem(Type)
	, World(InWorld)
	, ID(InWorld)
{
}

FWorldTreeItem::FWorldTreeItem(TWeakObjectPtr<UWorld> InWorld)
	: ISceneOutlinerTreeItem(Type)
	, World(InWorld)
	, ID(InWorld.Get())
{
}

FSceneOutlinerTreeItemID FWorldTreeItem::GetID() const
{
	return ID;
}

FString FWorldTreeItem::GetDisplayString() const
{
	if (UWorld* WorldPtr = World.Get())
	{
		return SceneOutliner::GetWorldDescription(WorldPtr).ToString();
	}
	return FString();
}

FString FWorldTreeItem::GetWorldName() const
{
	if (UWorld* WorldPtr = World.Get())
	{
		return World->GetFName().GetPlainNameString();
	}
	return FString();
}

bool FWorldTreeItem::CanInteract() const
{
	if (UWorld* WorldPtr = World.Get())
	{
		return Flags.bInteractive && WorldPtr->WorldType == EWorldType::Editor;
	}
	return Flags.bInteractive;
}

void FWorldTreeItem::GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner)
{
	auto SharedOutliner = StaticCastSharedRef<SSceneOutliner>(Outliner.AsShared());
	UWorld* WorldPtr = World.Get();

	if (WorldPtr)
	{
		// Invoke context menu providers
		for (TPair<FName, FContextMenuProviderFunc> ContextMenuProvider : ContextMenuProviders)
		{
			ContextMenuProvider.Value(WorldPtr, Menu);
		}
	}

	FToolMenuSection& Section = Menu->AddSection("Section");

	// Create Folder menu entry
	FSceneOutlinerMenuHelper::AddMenuEntryCreateFolder(Section, Outliner);

	const FSlateIcon FindInContentBrowserIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser");
	Section.AddMenuEntry("BrowseToWorld", LOCTEXT("BrowseToWorld", "Browse to Asset"), FText(), FindInContentBrowserIcon, FUIAction(FExecuteAction::CreateSP(this, &FWorldTreeItem::BrowseToAsset), FCanExecuteAction::CreateSP(this, &FWorldTreeItem::CanBrowseToAsset)));
	
	// Open world settings menu entry
	const FSlateIcon WorldSettingsIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.WorldProperties.Tab");
	Section.AddMenuEntry("OpenWorldSettings", LOCTEXT("OpenWorldSettings", "World Settings"), FText(), WorldSettingsIcon, FExecuteAction::CreateSP(this, &FWorldTreeItem::OpenWorldSettings));

	// Cleanup Folders menu entry
	ULevel* Level = WorldPtr ? WorldPtr->PersistentLevel.Get() : nullptr;
	FSceneOutlinerMenuHelper::AddMenuEntryCleanupFolders(Section, Level);
}

TSharedRef<SWidget> FWorldTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SWorldTreeLabel, *this, Outliner, InRow);
}

void FWorldTreeItem::OpenWorldSettings() const
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(FName("WorldSettingsTab"));	
}

void FWorldTreeItem::BrowseToAsset() const
{
	if (CanBrowseToAsset())
	{
		GEditor->SyncBrowserToObject(World.Get());
	}
}

bool FWorldTreeItem::CanBrowseToAsset() const
{
	if (UWorld* WorldPtr = World.Get())
	{
		return !WorldPtr->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor | PKG_NewlyCreated);
	}

	return false;
}

void FWorldTreeItem::RegisterContextMenuProvider(FName InProviderName, const FContextMenuProviderFunc& InProviderFunc)
{
	check(!ContextMenuProviders.Contains(InProviderName));
	ContextMenuProviders.Add(InProviderName, InProviderFunc);
}

void FWorldTreeItem::UnregisterContextMenuProvider(FName InProviderName)
{
	verify(ContextMenuProviders.Remove(InProviderName));
}

#undef LOCTEXT_NAMESPACE
