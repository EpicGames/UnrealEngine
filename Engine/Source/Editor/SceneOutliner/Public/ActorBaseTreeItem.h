// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISceneOutlinerTreeItem.h"
#include "SceneOutlinerStandaloneTypes.h"

#define UE_API SCENEOUTLINER_API

struct FGuid;

/** A tree item that represents an actor, loaded or unloaded */
struct IActorBaseTreeItem : ISceneOutlinerTreeItem
{
public:
	IActorBaseTreeItem(FSceneOutlinerTreeItemType InType) : ISceneOutlinerTreeItem(InType) {}

	/** Static type identifier for the base class tree item */
	static UE_API const FSceneOutlinerTreeItemType Type;

	virtual const FGuid& GetGuid() const =0;
};

#undef UE_API
