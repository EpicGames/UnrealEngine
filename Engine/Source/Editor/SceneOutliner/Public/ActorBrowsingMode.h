// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorModeInteractive.h"
#include "WorldPartition/WorldPartitionHandle.h"

class UActorBrowsingModeSettings;
class IWorldPartitionEditorModule;
class FWorldPartitionActorDescInstance;

class FActorBrowsingMode : public FActorModeInteractive
{
public:
	SCENEOUTLINER_API FActorBrowsingMode(SSceneOutliner* InSceneOutliner, TWeakObjectPtr<UWorld> InSpecifiedWorldToDisplay = nullptr);
	SCENEOUTLINER_API virtual ~FActorBrowsingMode();

	/* Begin ISceneOutlinerMode Interface */
	SCENEOUTLINER_API virtual void Rebuild() override;
	SCENEOUTLINER_API virtual FCreateSceneOutlinerMode CreateFolderPickerMode(const FFolder::FRootObject& InRootObject = FFolder::GetInvalidRootObject()) const override;
	SCENEOUTLINER_API virtual void InitializeViewMenuExtender(TSharedPtr<FExtender> Extender) override;
	SCENEOUTLINER_API virtual TSharedPtr<SWidget> CreateContextMenu() override;
	SCENEOUTLINER_API virtual void OnItemAdded(FSceneOutlinerTreeItemPtr Item) override;
	SCENEOUTLINER_API virtual void OnItemRemoved(FSceneOutlinerTreeItemPtr Item) override;
	SCENEOUTLINER_API virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;
	SCENEOUTLINER_API virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override;
	SCENEOUTLINER_API virtual void OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType) override;
	SCENEOUTLINER_API virtual void OnItemPassesFilters(const ISceneOutlinerTreeItem& Item) override;
	SCENEOUTLINER_API virtual FReply OnKeyDown(const FKeyEvent& InKeyEvent) override;
	SCENEOUTLINER_API virtual void OnDuplicateSelected() override;
	SCENEOUTLINER_API virtual bool HasCustomFolderDoubleClick() const override;
	SCENEOUTLINER_API virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const override;
	SCENEOUTLINER_API virtual FText GetStatusText() const override;
	SCENEOUTLINER_API virtual FSlateColor GetStatusTextColor() const override;
	virtual ESelectionMode::Type GetSelectionMode() const override { return ESelectionMode::Multi; }
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual bool ShouldShowFolders() const override { return true; }
	virtual bool SupportsCreateNewFolder() const override { return true; }
	virtual bool ShowStatusBar() const override { return true; }
	virtual bool ShowViewButton() const override { return true; }
	virtual bool ShowFilterOptions() const override { return true; }
	SCENEOUTLINER_API virtual bool CanDelete() const override;
	SCENEOUTLINER_API virtual bool CanRename() const override;
	SCENEOUTLINER_API virtual bool CanCut() const override;
	SCENEOUTLINER_API virtual bool CanCopy() const override;
	SCENEOUTLINER_API virtual bool CanPaste() const override;
	virtual bool CanSupportDragAndDrop() const { return true; }
	SCENEOUTLINER_API virtual bool HasErrors() const;
	SCENEOUTLINER_API virtual FText GetErrorsText() const;
	SCENEOUTLINER_API virtual void RepairErrors() const;	
	SCENEOUTLINER_API virtual void BindCommands(const TSharedRef<FUICommandList>& OutCommandList) override;
	SCENEOUTLINER_API virtual FFolder CreateNewFolder() override;
	SCENEOUTLINER_API virtual FFolder GetFolder(const FFolder& ParentPath, const FName& LeafName) override;
	SCENEOUTLINER_API virtual bool CreateFolder(const FFolder& NewFolder) override;
	SCENEOUTLINER_API virtual bool ReparentItemToFolder(const FFolder& FolderPath, const FSceneOutlinerTreeItemPtr& Item) override;
	SCENEOUTLINER_API virtual void SelectFoldersDescendants(const TArray<FFolderTreeItem*>& FolderItems, bool bSelectImmediateChildrenOnly) override;
	SCENEOUTLINER_API virtual void PinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) override;
	SCENEOUTLINER_API virtual void UnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) override;
	SCENEOUTLINER_API virtual bool CanPinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const override;
	SCENEOUTLINER_API virtual bool CanUnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const override;
	SCENEOUTLINER_API virtual void SynchronizeSelection() override;

	/* End ISceneOutlinerMode Interface */
public:
	/* External events this mode must respond to */

	/** Called by the engine when a component is updated */
	void OnComponentsUpdated();
	/** Called by the engine when an actor is deleted */
	void OnLevelActorDeleted(AActor* Actor);

	/** Called by the editor to allow selection of unloaded actors */
	void OnSelectUnloadedActors(const TArray<FGuid>& ActorGuids);
	
	/** Called when an actor desc is removed */
	UE_DEPRECATED(5.4, "Use OnActorDescInstanceRemoved")
	void OnActorDescRemoved(FWorldPartitionActorDesc* InActorDesc) {}

	/** Called when an actor desc instance is removed */
	void OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* InActorDescInstance);

	/** Called by engine when edit cut actors begins */
	void OnEditCutActorsBegin();

	/** Called by engine when edit cut actors ends */
	void OnEditCutActorsEnd();

	/** Called by engine when edit copy actors begins */
	void OnEditCopyActorsBegin();

	/** Called by engine when edit copy actors ends */
	void OnEditCopyActorsEnd();

	/** Called by engine when edit paste actors begins */
	SCENEOUTLINER_API virtual void OnEditPasteActorsBegin();

	/** Called by engine when edit paste actors ends */
	SCENEOUTLINER_API virtual void OnEditPasteActorsEnd();

	/** Called by engine when edit duplicate actors begins */
	SCENEOUTLINER_API virtual void OnDuplicateActorsBegin();

	/** Called by engine when edit duplicate actors ends */
	SCENEOUTLINER_API virtual void OnDuplicateActorsEnd();

	/** Called by engine when edit delete actors begins */
	void OnDeleteActorsBegin();

	/** Called by engine when edit delete actors ends */
	void OnDeleteActorsEnd();
	
	/** Function called by the Outliner Filter Bar to compare an item with Type Filters*/
	SCENEOUTLINER_API virtual bool CompareItemWithClassName(SceneOutliner::FilterBarType InItem, const TSet<FTopLevelAssetPath>&) const override;
	
	/** Check whether the Rename can be executed */
	SCENEOUTLINER_API virtual bool CanExecuteRename();
private:
	/** Build and up the context menu */
	TSharedPtr<SWidget> BuildContextMenu();
	/** Register the context menu with the engine */
	static void RegisterContextMenu();
	static void FillDefaultContextBaseMenu(UToolMenu* InMenu);
	bool CanPasteFoldersOnlyFromClipboard() const;

	/** Delete the current selection */
	void OnExecuteDelete();

	/** Refresh the Outliner */
	void OnExecuteRefresh();

	/** Rename the selected Item (Only work with 1 selected Item) */
	void OnExecuteRename();

	void SynchronizeSelectedActorDescs();

	void OnActorEditorContextSubsystemChanged();

	/** Filter factories */
	static TSharedRef<FSceneOutlinerFilter> CreateShowOnlySelectedActorsFilter();
	static TSharedRef<FSceneOutlinerFilter> CreateIsInCurrentLevelFilter();
	static TSharedRef<FSceneOutlinerFilter> CreateIsInCurrentDataLayersFilter();
	static TSharedRef<FSceneOutlinerFilter> CreateHideComponentsFilter();
	static TSharedRef<FSceneOutlinerFilter> CreateHideUnloadedActorsFilter();
	static TSharedRef<FSceneOutlinerFilter> CreateHideEmptyFoldersFilter();
	TSharedRef<FSceneOutlinerFilter> CreateIsInCurrentContentBundleFilter();

	/** Functions to expose selection framing to the UI */
	void OnToggleAlwaysFrameSelection();
	bool ShouldAlwaysFrameSelection() const;

	void OnToggleHideTemporaryActors();
	bool ShouldHideTemporaryActors() const;

	void OnToggleHideLevelInstanceHierarchy();
	bool ShouldHideLevelInstanceHierarchy() const;

	void OnToggleFolderDoubleClickMarkCurrentFolder();
	bool DoesFolderDoubleClickMarkCurrentFolder() const;

	void OnToggleShouldUpdateContentWhileInPIEFocused();
	bool ShouldUpdateContentWhileInPIEFocused() const;

	void OnToggleCollapseOutlinerTreeOnNewSelection();
	bool CollapseOutlinerTreeOnNewSelection() const;

	/**
	 * Get a mutable version of the ActorBrowser config for setting values.
	 * @returns		The config for this ActorBrowser.
	 * @note		If OutlinerIdentifier is not set for this outliner, it is not possible to store settings.
	 */
	struct FActorBrowsingModeConfig* GetMutableConfig();

	/**
	 * Get a const version of the ActorBrowser config for getting values.
	 * @returns		The config for this ActorBrowser.
	 * @note		If OutlinerIdentifier is not set for this outliner, it is not possible to store settings.
	 */
	const FActorBrowsingModeConfig* GetConstConfig() const;

	/** Save the config for this ActorBrowser */
	void SaveConfig();
	
private:

	IWorldPartitionEditorModule* WorldPartitionEditorModule;
	/** Number of actors (including unloaded) which have passed through the filters */
	uint32 FilteredActorCount = 0;
	/** Number of unloaded actors which have passed through all the filters */
	uint32 FilteredUnloadedActorCount = 0;
	/** List of unloaded actors which passed through the regular filters and may or may not have passed the search filter */
	TSet<FWorldPartitionHandle> ApplicableUnloadedActors;
	/** List of actors which passed the regular filters and may or may not have passed the search filter */
	TSet<TWeakObjectPtr<AActor>> ApplicableActors;

	bool bRepresentingWorldGameWorld = false;
	bool bRepresentingWorldPartitionedWorld = false;
};
