// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "ActorBaseTreeItem.h"
#include "UObject/ObjectKey.h"
#include "WorldPartition/WorldPartitionHandle.h"

#define UE_API SCENEOUTLINER_API

class UActorDescContainerInstance;
class UExternalDataLayerAsset;
class FWorldPartitionActorDesc;
class UToolMenu;

/** A tree item that represents an actor in the world */
struct FActorDescTreeItem : IActorBaseTreeItem
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const FWorldPartitionActorDescInstance*);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const FWorldPartitionActorDescInstance*);

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(*ActorDescHandle);
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(*ActorDescHandle);
	}

	/** The actor desc this tree item is associated with. */
	FWorldPartitionHandle ActorDescHandle;

	/** Constant identifier for this tree item */
	FSceneOutlinerTreeItemID ID;

	static UE_API FSceneOutlinerTreeItemID ComputeTreeItemID(FGuid InActorGuid, UActorDescContainerInstance* InContainer);

	static UE_API bool ShouldDisplayInOutliner(const FWorldPartitionActorDescInstance* InActorDescInstance);

	UE_DEPRECATED(5.4, "Use FWorldPartitionActorDescInstance version instead")
	static bool ShouldDisplayInOutliner(const FWorldPartitionActorDesc* ActorDesc) { return false;}

	/** Static type identifier for this tree item class */
	static UE_API const FSceneOutlinerTreeItemType Type;

	/** Construct this item from an actor desc */
	UE_API FActorDescTreeItem(const FGuid& InActorGuid, UActorDescContainerInstance* InContainerInstance);

	/** Construct this item from an actor desc instance */
	UE_API FActorDescTreeItem(const FWorldPartitionActorDescInstance* InActorDescInstance);

	UE_DEPRECATED(5.4, "Use FWorldPartitionActorDescInstance version instead")
	FActorDescTreeItem(const FGuid& InActorGuid, class UActorDescContainer* InContainer) : IActorBaseTreeItem(Type) {}

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return *ActorDescHandle != nullptr; }
	UE_API virtual FSceneOutlinerTreeItemID GetID() const override;
	UE_API virtual FString GetDisplayString() const override;
	UE_API virtual bool CanInteract() const override;
	UE_API virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	UE_API virtual void GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner) override;
	virtual bool HasVisibilityInfo() const override { return true; }
	UE_API virtual bool GetVisibility() const override;
	UE_API virtual bool ShouldShowPinnedState() const override;
	virtual bool ShouldShowVisibilityState() const override { return false; }
	virtual bool HasPinnedStateInfo() const override { return true; }
	UE_API virtual bool GetPinnedState() const override;
	UE_API virtual FString GetPackageName() const override;
	/* End ISceneOutlinerTreeItem Implementation */
	
	/* Begin IActorBaseTreeItem Implementation */
	virtual const FGuid& GetGuid() const override { return ActorGuid; }
	/* End IActorBaseTreeItem Implementation */

	UE_API UExternalDataLayerAsset* GetExternalDataLayerAsset() const;

	UE_API void FocusActorBounds() const;

protected:
	FString DisplayString;

private:
	mutable TSoftObjectPtr<UExternalDataLayerAsset> CachedExternalDataLayerAsset;

	UE_API void CopyActorFilePathtoClipboard() const;
	FGuid ActorGuid;
};

#undef UE_API
