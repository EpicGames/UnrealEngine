// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISceneOutlinerHierarchy.h"

class FActorFolderHierarchy : public ISceneOutlinerHierarchy
{
public:
	FActorFolderHierarchy(ISceneOutlinerMode* InMode, const TWeakObjectPtr<UWorld>& World, const FFolder::FRootObject& InRootObject = FFolder::GetInvalidRootObject());
	virtual ~FActorFolderHierarchy() = default;


	/* Begin ISceneOutlinerHierarchy Implementation */
	virtual void CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
	virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const override;
	virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) override;
	/* End ISceneOutlinerHierarchy Implementation */
private:
	/** Adds all the direct and indirect children of a world to OutItems */
	void CreateWorldChildren(UWorld* World, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;
private:
	/** The world which this hierarchy is representing */
	TWeakObjectPtr<UWorld> RepresentingWorld;

	FFolder::FRootObject RootObject;
};
