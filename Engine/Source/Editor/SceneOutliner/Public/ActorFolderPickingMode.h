// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorMode.h"

#define UE_API SCENEOUTLINER_API

class FActorFolderPickingMode : public FActorMode
{
public:
	UE_API FActorFolderPickingMode(SSceneOutliner* InSceneOutliner, FOnSceneOutlinerItemPicked InOnItemPicked, TWeakObjectPtr<UWorld> InSpecifiedWorldToDisplay = nullptr, const FFolder::FRootObject& InRootObject = FFolder::GetInvalidRootObject());
	virtual ~FActorFolderPickingMode() = default;

	/* Begin ISceneOutlinerMode Implementation */
	UE_API virtual void OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType) override;
	UE_API virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;
	UE_API virtual void CreateViewContent(FMenuBuilder& MenuBuilder) override;
	virtual bool ShowViewButton() const override { return true; }
	virtual bool ShouldShowFolders() const { return true; }
protected:
	UE_API virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() override;
	/* End ISceneOutlinerMode Implementation */

	/** Delegate to call when an item is picked */
	FOnSceneOutlinerItemPicked OnItemPicked;

	FFolder::FRootObject RootObject;
};

#undef UE_API
