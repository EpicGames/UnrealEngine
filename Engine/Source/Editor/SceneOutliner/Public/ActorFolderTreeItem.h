// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FolderTreeItem.h"
#include "ActorFolder.h"

#define UE_API SCENEOUTLINER_API

struct FActorFolderTreeItem : public FFolderTreeItem
{
public:
	/** Static type identifier for this tree item class */
	static UE_API const FSceneOutlinerTreeItemType Type;

	UE_API FActorFolderTreeItem(const FFolder& InFolder, const TWeakObjectPtr<UWorld>& InWorld);

	/** The world which this folder belongs to */
	TWeakObjectPtr<UWorld> World;

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return World.IsValid(); }
	UE_API virtual void OnExpansionChanged() override;
	UE_API virtual void Delete(const FFolder& InNewParentFolder) override;
	UE_API virtual bool CanInteract() const override;
	UE_API virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	virtual bool ShouldShowPinnedState() const override { return false; }
	UE_API virtual bool ShouldShowVisibilityState() const override;
	UE_API virtual bool ShouldRemoveOnceLastChildRemoved() const override;
	UE_API virtual FFolder GetFolder() const override;
	UE_API virtual FString GetPackageName() const override;
	/* End FFolderTreeItem Implementation */
		
	/* Begin FFolderTreeItem Implementation */
	UE_API virtual void MoveTo(const FFolder& InNewParentFolder) override;
	UE_API virtual void SetPath(const FName& InNewPath) override;
	
	UE_API bool CanChangeChildrenPinnedState() const;
	const UActorFolder* GetActorFolder() const { return ActorFolder.Get(); }
private:
	UE_API virtual void CreateSubFolder(TWeakPtr<SSceneOutliner> WeakOutliner) override;
	/* End FFolderTreeItem Implementation */

	/** The actor folder object (can be invalid) */
	TWeakObjectPtr<UActorFolder> ActorFolder;
};

#undef UE_API
