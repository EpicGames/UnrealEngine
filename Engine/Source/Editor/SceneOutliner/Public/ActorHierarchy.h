// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISceneOutlinerHierarchy.h"
#include "Folder.h"

class FWorldPartitionActorDesc;
class FWorldPartitionActorDescInstance;
class UActorFolder;
class UWorldPartition;

class FActorHierarchy : public ISceneOutlinerHierarchy
{
public:
	SCENEOUTLINER_API virtual ~FActorHierarchy();

	static TUniquePtr<FActorHierarchy> Create(ISceneOutlinerMode* Mode, const TWeakObjectPtr<UWorld>& World);

	/** Create a linearization of all applicable items in the hierarchy */
	SCENEOUTLINER_API virtual void CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
	/** Create a linearization of all direct and indirect children of a given item in the hierarchy */
	SCENEOUTLINER_API virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const override;
	/** Forcibly create a parent item for a given tree item */
	SCENEOUTLINER_API virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) override;

	void SetShowingComponents(bool bInShowingComponents) { bShowingComponents = bInShowingComponents; }
	void SetShowingOnlyActorWithValidComponents(bool bInShowingOnlyActorWithValidComponents) { bShowingOnlyActorWithValidComponents = bInShowingOnlyActorWithValidComponents; }
	void SetShowingLevelInstances(bool bInShowingLevelInstances) { bShowingLevelInstances = bInShowingLevelInstances; }
	void SetShowingUnloadedActors(bool bInShowingUnloadedActors) { bShowingUnloadedActors = bInShowingUnloadedActors; }
	void SetShowingEmptyFolders(bool bInShowingEmptyFolders) { bShowingEmptyFolders = bInShowingEmptyFolders; }
	void SetSearchComponentsByActorName(bool bInSearchComponentsByActorName) { bSearchComponentsByActorName = bInSearchComponentsByActorName;}

private:
	bool IsShowingUnloadedActors() const;
	/** Adds all the direct and indirect children of a world to OutItems */
	void CreateWorldChildren(UWorld* World, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;

protected:
	// Update the hierarchy when actor or world changing events occur
	void OnWorldPartitionCreated(UWorld* InWorld);

	/** Create all component items for an actor if we are showing components and place them in OutItems */
	SCENEOUTLINER_API void CreateComponentItems(const AActor* Actor, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;
	
	SCENEOUTLINER_API virtual void OnLevelActorAdded(AActor* InActor);
	SCENEOUTLINER_API virtual void OnLevelActorDeleted(AActor* InActor);
		
	SCENEOUTLINER_API virtual void OnLevelActorAttached(AActor* InActor, const AActor* InParent);
	SCENEOUTLINER_API virtual void OnLevelActorDetached(AActor* InActor, const AActor* InParent);

	SCENEOUTLINER_API virtual void OnLevelActorFolderChanged(const AActor* InActor, FName OldPath);

	SCENEOUTLINER_API virtual void OnLoadedActorAdded(AActor& InActor);
	SCENEOUTLINER_API virtual void OnLoadedActorRemoved(AActor& InActor);

	SCENEOUTLINER_API virtual void OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* InActorDescInstance);
	SCENEOUTLINER_API virtual void OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* InActorDescInstance);

	UE_DEPRECATED(5.4, "Use OnActorDescInstanceAdded instead")
	virtual void OnActorDescAdded(FWorldPartitionActorDesc* ActorDesc) {}

	UE_DEPRECATED(5.4, "Use OnActorDescInstanceRemoved instead")
	virtual void OnActorDescRemoved(FWorldPartitionActorDesc* ActorDesc) {}
	
	void OnComponentsUpdated();

	void OnLevelActorListChanged();
	
	SCENEOUTLINER_API virtual void OnActorFolderAdded(UActorFolder* InActorFolder);
	void OnActorFoldersUpdatedEvent(ULevel* InLevel);
		
	SCENEOUTLINER_API virtual void OnLevelAdded(ULevel* InLevel, UWorld* InWorld);
	SCENEOUTLINER_API virtual void OnLevelRemoved(ULevel* InLevel, UWorld* InWorld);

	void OnWorldPartitionInitialized(UWorldPartition* InWorldPartition);
	void OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition);

	/** Called when a folder is to be created */
	SCENEOUTLINER_API virtual void OnBroadcastFolderCreate(UWorld& InWorld, const FFolder& InNewFolder);

	/** Called when a folder is to be moved */
	SCENEOUTLINER_API virtual void OnBroadcastFolderMove(UWorld& InWorld, const FFolder& InOldFolder, const FFolder& InNewFolder);

	/** Called when a folder is to be deleted */
	SCENEOUTLINER_API virtual void OnBroadcastFolderDelete(UWorld& InWorld, const FFolder& InFolder);

	// Remove the unloaded actor item for the given actor
	SCENEOUTLINER_API void RemoveActorDesc(AActor& InActor);
	
	// Add the unloaded actor item for the given actor
	SCENEOUTLINER_API void AddActorDesc(AActor& InActor);

	// Remove all Actor Folders belonging to this level
	SCENEOUTLINER_API void RemoveLevelActorFolders(ULevel* InLevel, UWorld* InWorld);

	// Create items for Actor Folders
	SCENEOUTLINER_API void CreateFolderItems(UWorld* InWorld, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;

	// Create items for unloaded actors
	SCENEOUTLINER_API void CreateUnloadedItems(UWorld* World, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;

	// Insert the given item at the right position in the tree, and optionally create and insert components if needed
	SCENEOUTLINER_API void InsertActorItemAndCreateComponents(AActor* InActor, FSceneOutlinerTreeItemPtr ActorItem, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;

	// Check if the actor we are creating an item for is in the current level instance, if we are currently editing one
	SCENEOUTLINER_API bool CheckLevelInstanceEditing(UWorld* World, AActor* Actor) const;

	// Create a SceneOutlinerTreeItem for the given actor
	SCENEOUTLINER_API virtual FSceneOutlinerTreeItemPtr CreateItemForActor(AActor* InActor, bool bForce = false) const;

	// Create a SceneOutlinerTreeItem for the given actor descriptor instance
	SCENEOUTLINER_API virtual FSceneOutlinerTreeItemPtr CreateItemForActorDescInstance(const FWorldPartitionActorDescInstance* InActorDescInstance, bool bForce = false) const;

	SCENEOUTLINER_API virtual FSceneOutlinerTreeItemPtr CreateItemForFolder(const FFolder& InFolder, const TWeakObjectPtr<UWorld>& InWorld, bool bForce = false) const;
	
protected:
	/** Send a an event indicating a full refresh of the hierarchy is required */
	void FullRefreshEvent();

	TWeakObjectPtr<UWorld> RepresentingWorld;
	
	bool bShowingComponents = false;
	bool bShowingOnlyActorWithValidComponents = false;
	bool bShowingLevelInstances = false;
	bool bShowingUnloadedActors = false;
	bool bShowingEmptyFolders = false;
	bool bSearchComponentsByActorName = false;

protected:

	SCENEOUTLINER_API static void Create_Internal(FActorHierarchy* Hierarchy, const TWeakObjectPtr<UWorld>& World);
	
	SCENEOUTLINER_API FActorHierarchy(ISceneOutlinerMode* Mode, const TWeakObjectPtr<UWorld>& Worlds);

	FActorHierarchy(const FActorHierarchy&) = delete;
	FActorHierarchy& operator=(const FActorHierarchy&) = delete;
};
