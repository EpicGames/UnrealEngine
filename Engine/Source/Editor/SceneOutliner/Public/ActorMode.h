// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerMode.h"
#include "WorldPartition/WorldPartitionHandle.h"

#define UE_API SCENEOUTLINER_API

namespace SceneOutliner
{
	/** Functor which can be used to get weak actor pointers from a selection */
	struct FWeakActorSelector
	{
		UE_API bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, TWeakObjectPtr<AActor>& DataOut) const;
	};

	/** Functor which can be used to get actors from a selection including component parents */
	struct FActorSelector
	{
		UE_API bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, AActor*& ActorPtrOut) const;
	};

	struct UE_DEPRECATED(5.4, "Use FActorHandleSelector instead") FActorDescSelector
	{
		bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>&Item, FWorldPartitionActorDesc * &ActorDescPtrOut) const { return false; }
	};

	/** Functor which can be used to get actor descriptors from a selection  */
	struct FActorHandleSelector
	{
		UE_API bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, FWorldPartitionHandle& ActorHandleOut) const;
	};
}

struct FActorModeParams
{
	FActorModeParams() {}

	FActorModeParams(SSceneOutliner* InSceneOutliner, const TWeakObjectPtr<UWorld>& InSpecifiedWorldToDisplay = nullptr, bool bInHideComponents = true,
		bool bInHideLevelInstanceHierarchy = true, bool bInHideUnloadedActors = true, bool bInHideEmptyFolders = true,
		bool bInCanInteractWithSelectableActorsOnly = true, bool binSearchComponentsByActorName = false)
		: SpecifiedWorldToDisplay(InSpecifiedWorldToDisplay)
		, SceneOutliner(InSceneOutliner)
		, bHideComponents(bInHideComponents)
		, bHideLevelInstanceHierarchy(bInHideLevelInstanceHierarchy)
		, bHideUnloadedActors(bInHideUnloadedActors)
		, bHideEmptyFolders(bInHideEmptyFolders)
		, bCanInteractWithSelectableActorsOnly(bInCanInteractWithSelectableActorsOnly)
		, bSearchComponentsByActorName(binSearchComponentsByActorName)
	{}

	TWeakObjectPtr<UWorld> SpecifiedWorldToDisplay = nullptr;
	SSceneOutliner* SceneOutliner = nullptr;
	bool bHideComponents = true;
	bool bHideActorWithNoComponent = false;
	bool bHideLevelInstanceHierarchy = true;
	bool bHideUnloadedActors = true;
	bool bHideEmptyFolders = true;
	bool bCanInteractWithSelectableActorsOnly = true;
	bool bShouldUpdateContentWhileInPIEFocused = false;
	bool bSearchComponentsByActorName = false;
};

class FActorMode : public ISceneOutlinerMode
{
public:
	struct EItemSortOrder
	{
		enum Type { World = 0, Level = 10, Folder = 20, Actor = 30 };
	};
	
	UE_API FActorMode(const FActorModeParams& Params);
	UE_API virtual ~FActorMode();

	UE_API virtual void Rebuild() override;

	UE_API void BuildWorldPickerMenu(FMenuBuilder& MenuBuilder);

	virtual void SynchronizeSelection() override { SynchronizeActorSelection(); }

	UE_API virtual void OnFilterTextChanged(const FText& InFilterText) override;

	UE_API virtual int32 GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const override;

	static UE_API bool IsActorDisplayable(const SSceneOutliner* SceneOutliner, const AActor* Actor, bool bShowLevelInstanceContent = false);
	static UE_API bool IsActorLevelDisplayable(ULevel* InLevel);

	UE_API virtual FFolder::FRootObject GetRootObject() const override;
	UE_API virtual FFolder::FRootObject GetPasteTargetRootObject() const override;

	UE_API virtual bool CanInteract(const ISceneOutlinerTreeItem& Item) const override;
	UE_API virtual bool CanPopulate() const override;
	
	UE_API virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const override;
	UE_API virtual bool ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const override;
	UE_API virtual FSceneOutlinerDragValidationInfo ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const override;
	UE_API virtual void OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const override;


private:
	/** Called when the user selects a world in the world picker menu */
	UE_API void OnSelectWorld(TWeakObjectPtr<UWorld> World);
private:
	/* Private Helpers */

	UE_API void ChooseRepresentingWorld();
	UE_API bool IsWorldChecked(TWeakObjectPtr<UWorld> World) const;
	UE_API bool GetFolderNamesFromPayload(const FSceneOutlinerDragDropPayload& InPayload, TArray<FName>& OutFolders, FFolder::FRootObject& OutCommonRootObject) const;

protected:
	UE_API void SynchronizeActorSelection();
	UE_API virtual bool IsActorDisplayable(const AActor* InActor) const;

	/** Set the Scene Outliner attached to this mode as the most recently used outliner in the Level Editor */
	UE_API void SetAsMostRecentOutliner() const;

	UE_API virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() override;

	// Called when actors are attached to a parent actor via drag and drop
	virtual void OnActorsAttached(AActor* ParentActor, TArray<TWeakObjectPtr<AActor>> ChildActors) const {}
	
	UE_API FFolder GetWorldDefaultRootFolder() const;
protected:
	/** The world which we are currently representing */
	TWeakObjectPtr<UWorld> RepresentingWorld;
	/** The world which the user manually selected */
	TWeakObjectPtr<UWorld> UserChosenWorld;

	/** If this mode was created to display a specific world, don't allow it to be reassigned */
	const TWeakObjectPtr<UWorld> SpecifiedWorldToDisplay;

	/** Should components be hidden */
	bool bHideComponents;
	/** Should actor with no component be hidden. */
	bool bHideActorWithNoComponent;
	/** Should the level instance hierarchy be hidden */
	bool bHideLevelInstanceHierarchy;
	/** Should unloaded actors be hidden */
	bool bHideUnloadedActors;
	/** Should empty folders be hidden */
	bool bHideEmptyFolders;
	/** Should the outliner scroll to the item on selection */
	bool bAlwaysFrameSelection;
	/** If True, CanInteract will be restricted to selectable actors only. */
	bool bCanInteractWithSelectableActorsOnly;
	/** Should we update content when in PIE and the PIE viewport has focus. */
	bool bShouldUpdateContentWhileInPIEFocused;
	/** If true and bHideComponents is false, components will be shown if the owning actor is searched for even if the search text does not match
	 * the components
	 */
	bool bSearchComponentsByActorName;
	/** Should Outliner Tree get collapsed on selection, except for the item that was just selected */
	bool bCollapseOutlinerTreeOnNewSelection;
};

#undef UE_API
