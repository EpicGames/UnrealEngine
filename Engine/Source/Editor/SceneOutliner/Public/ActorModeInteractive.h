// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorMode.h"

class FActorModeInteractive : public FActorMode
{
public:
	SCENEOUTLINER_API FActorModeInteractive(const FActorModeParams& Params);
	SCENEOUTLINER_API virtual ~FActorModeInteractive();

	virtual bool IsInteractive() const override { return true; }
protected:
	/* Events */

	void OnMapChange(uint32 MapFlags);
	void OnNewCurrentLevel();

	SCENEOUTLINER_API virtual void OnActorLabelChanged(AActor* ChangedActor);
	
	void OnLevelSelectionChanged(UObject* Obj);
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);
	void OnLevelActorRequestsRename(const AActor* Actor);
	void OnPostLoadMapWithWorld(UWorld* World);
};
