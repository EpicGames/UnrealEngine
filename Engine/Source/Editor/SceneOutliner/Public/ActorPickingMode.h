// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorMode.h"

#define UE_API SCENEOUTLINER_API

class FActorPickingMode : public FActorMode
{
public:
	UE_API FActorPickingMode(const FActorModeParams& Params, FOnSceneOutlinerItemPicked OnItemPickedDelegate);

	virtual ~FActorPickingMode() = default;
public:
	UE_API virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;

	/** Allow the user to commit their selection by pressing enter if it is valid */
	UE_API virtual void OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType) override;

	UE_API virtual void CreateViewContent(FMenuBuilder& MenuBuilder) override;

	// Don't synchronize world selection
	virtual void SynchronizeSelection() override {};
public:
	virtual bool ShowViewButton() const override { return true; }
private:
	FOnSceneOutlinerItemPicked OnItemPicked;
};

#undef UE_API
