// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "ActorBaseTreeItem.h"
#include "UObject/ObjectKey.h"

#define UE_API SCENEOUTLINER_API

/** A tree item that represents an actor in the world */
struct FActorTreeItem : IActorBaseTreeItem
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const AActor*);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const AActor*);
		
	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(Actor.Get());
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const 
	{
		return Pred.Execute(Actor.Get());
	}

	/** The actor this tree item is associated with. */
	mutable TWeakObjectPtr<AActor> Actor;

	/** Constant identifier for this tree item */
	const FObjectKey ID;

	/** Static type identifier for this tree item class */
	static UE_API const FSceneOutlinerTreeItemType Type;

	/** Construct this item from an actor */
	UE_API FActorTreeItem(AActor* InActor);

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return Actor.IsValid(); }
	UE_API virtual FSceneOutlinerTreeItemID GetID() const override;
	UE_API virtual FFolder::FRootObject GetRootObject() const override;
	UE_API virtual FString GetDisplayString() const override;
	UE_API virtual bool CanInteract() const override;
	UE_API virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	UE_API virtual void OnVisibilityChanged(const bool bNewVisibility) override;
	virtual bool HasVisibilityInfo() const override { return true; }
	UE_API virtual bool GetVisibility() const override;
	UE_API virtual bool ShouldShowPinnedState() const override;
	UE_API virtual bool ShouldShowVisibilityState() const override;
	virtual bool HasPinnedStateInfo() const override { return true; };
	UE_API virtual bool GetPinnedState() const override;
	UE_API virtual void OnLabelChanged() override;
	UE_API virtual void GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner) override;
	UE_API virtual FString GetPackageName() const override;
	/* End ISceneOutlinerTreeItem Implementation */

	/* Begin IActorBaseTreeItem Implementation */
	UE_API virtual const FGuid& GetGuid() const override;
	/* End IActorBaseTreeItem Implementation */

	/** true if this item exists in both the current world and PIE. */
	bool bExistsInCurrentWorldAndPIE;

protected:
	/** For use by derived classes, as it allows for passing down their own type. */
	UE_API FActorTreeItem(FSceneOutlinerTreeItemType TypeIn, AActor* InActor);

	UE_API virtual void UpdateDisplayString();

	/** Cached display string */
	FString DisplayString;

private:
	UE_API void UpdateDisplayStringInternal();
};

#undef UE_API
