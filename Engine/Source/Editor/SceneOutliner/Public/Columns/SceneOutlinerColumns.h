// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "SceneOutlinerPublicTypes.h"
#include "Templates/SharedPointer.h"

#include "SceneOutlinerColumns.generated.h"

class ISceneOutliner;
struct ISceneOutlinerTreeItem;

// Column used to store a reference to the Scene Outliner owning or on a specific row
USTRUCT(meta = (DisplayName = "Outliner"))
struct FSceneOutlinerColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TWeakPtr<ISceneOutliner> Outliner;
};

namespace UE::Editor::Outliner
{
	/** The mapping domain used when registering outliner rows in TEDS. */
	inline const FName OutlinerMappingDomain = TEXT("TedsOutliner");

	/**
	 * Delegate to resolve a TEDS RowHandle from a SceneOutliner tree item. Bound on the outliner row.
	 * Custom outliner columns can query FTedsOutlinerRowHandleDealiaserColumn from the outliner row
	 * to map tree items to TEDS rows without depending on TedsOutliner internals.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(UE::Editor::DataStorage::RowHandle, FGetRowHandleFromOutlinerItem, const ISceneOutlinerTreeItem&);
}

/**
 * Column on the outliner row providing a pre-bound delegate to resolve a RowHandle from a legacy
 * ISceneOutlinerTreeItem. Custom outliner columns can query this from the outliner row to map tree
 * items to TEDS rows without directly depending on TedsOutliner internals.
 */
USTRUCT()
struct FTedsOutlinerRowHandleDealiaserColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::Outliner::FGetRowHandleFromOutlinerItem GetRowHandle;
};