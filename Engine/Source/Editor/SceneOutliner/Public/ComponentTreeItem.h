// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "ISceneOutlinerTreeItem.h"
#include "UObject/ObjectKey.h"
#include "Components/SceneComponent.h"

#define UE_API SCENEOUTLINER_API

/** A tree item that represents an Component in the world */
struct FComponentTreeItem : ISceneOutlinerTreeItem
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const UActorComponent*);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const UActorComponent*);

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(Component.Get());
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(Component.Get());
	}

	/** The Component this tree item is associated with. */
	mutable TWeakObjectPtr<UActorComponent> Component;

	/** Constant identifier for this tree item */
	const FObjectKey ID;

	/** Static type identifier for this tree item class */
	static UE_API const FSceneOutlinerTreeItemType Type;

	/** Construct this item from an Component */
	UE_API FComponentTreeItem(UActorComponent* InComponent, bool bInSearchComponentsByActorName = false);

	UE_API bool GetSearchComponentByActorName() const;

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return Component.IsValid(); }
	UE_API virtual FSceneOutlinerTreeItemID GetID() const override;
	UE_API virtual FString GetDisplayString() const override;
	UE_API virtual bool CanInteract() const override;
	UE_API virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	virtual bool ShouldShowVisibilityState() const { return false; }
	virtual bool HasVisibilityInfo() const { return false; }
	virtual bool GetVisibility() const { return false; }
	/* End ISceneOutlinerTreeItem Implementation */
public:
	/** true if this item exists in both the current world and PIE. */
	bool bExistsInCurrentWorldAndPIE;

	/** If true components will be shown if the owning actor is searched for even if the search text does not match the component */
	bool bSearchComponentsByActorName;
	
	/** Cache the string displayed */
	FString CachedDisplayString;
};

#undef UE_API
