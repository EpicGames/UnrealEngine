// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"
#include "Folder.h"
#include "ISceneOutlinerTreeItem.h"

#define UE_API SCENEOUTLINER_API

class UToolMenu;

namespace SceneOutliner
{
	struct FFolderPathSelector
	{
		bool operator()(TWeakPtr<ISceneOutlinerTreeItem> Item, FFolder& DataOut) const;
	};
}	// namespace SceneOutliner


/** A tree item that represents a folder in the world */
struct FFolderTreeItem : ISceneOutlinerTreeItem
{
public:
	/** Static type identifier for this tree item class */
	static UE_API const FSceneOutlinerTreeItemType Type;

	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const FFolder&);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const FFolder&);

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(GetFolder());
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(GetFolder());
	}

	/** The path of this folder. / separated. */
	FName Path;

	/** The leaf name of this folder */
	FName LeafName;

	UE_API FFolderTreeItem(const FFolder& InFolder, FSceneOutlinerTreeItemType InType);
	UE_API FFolderTreeItem(const FFolder& InFolder);
	/** Constructor that takes a path to this folder (including leaf-name) */
	UE_API FFolderTreeItem(FName InPath);
	/** Constructor that takes a path to this folder and a subclass tree item type (used for subclassing FFolderTreeItem) */
	UE_API FFolderTreeItem(FName InPath, FSceneOutlinerTreeItemType Type);

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return true; }
	UE_API virtual FSceneOutlinerTreeItemID GetID() const override;
	UE_API virtual FFolder::FRootObject GetRootObject() const override;
	UE_API virtual FString GetDisplayString() const override;
	UE_API virtual bool CanInteract() const override;
	UE_API virtual void GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner) override;
	/** Delete this folder, children will be reparented to provided new parent path */
	virtual void Delete(const FFolder& InNewParentFolder) {}
	
	UE_API virtual void SetPath(const FName& InNewPath);
	const FName& GetPath() const { return Path; }
	const FName& GetLeafName() const { return LeafName; }
	virtual FFolder GetFolder() const { return FFolder(RootObject, Path); }
	
	virtual bool ShouldShowPinnedState() const override { return true; }
	virtual bool HasPinnedStateInfo() const override { return false; }
	/* End ISceneOutlinerTreeItem Implementation */

	/** Move this folder to a new parent */
	virtual void MoveTo(const FFolder& InNewParentFolder) {}
private:
	/** Create a new folder as a child of this one */
	virtual void CreateSubFolder(TWeakPtr<SSceneOutliner> WeakOutliner) {}
	/** Duplicate folder hierarchy */
	UE_API void DuplicateHierarchy(TWeakPtr<SSceneOutliner> WeakOutliner);

	/** Folder's root object (can be null) */
	FFolder::FRootObject RootObject;
};

#undef UE_API
