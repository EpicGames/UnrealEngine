// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "SceneOutlinerFwd.h"
#include "SceneOutlinerPublicTypes.h"
#include "Filters/FilterBase.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"

// Forward declaration
template<typename ItemType> class STreeView;
struct FSceneOutlinerColumnInfo;

/**
 * The public interface for the Scene Outliner widget
 */
class ISceneOutliner : public SCompoundWidget
{
public:

	/** Sends a requests to the Scene Outliner to refresh itself the next chance it gets */
	virtual void Refresh() = 0;

	/** Tells the scene outliner that it should do a full refresh, which will clear the entire tree and rebuild it from scratch. */
	virtual void FullRefresh() = 0;

	/** @return Returns a string to use for highlighting results in the outliner list */
	virtual TAttribute<FText> GetFilterHighlightText() const = 0;

	/** @return Returns the common data for this outliner */
	virtual const FSharedSceneOutlinerData& GetSharedData() const = 0;

	/** Get a const reference to the actual tree hierarchy */
	virtual const STreeView<FSceneOutlinerTreeItemPtr>& GetTree() const = 0;

	/** Set the keyboard focus to the outliner */
	virtual void SetKeyboardFocus() = 0;

	/** Gets the cached icon for this class name */
	virtual const FSlateBrush* GetCachedIconForClass(FName InClassName) const = 0;

	/** Sets the cached icon for this class name */
	virtual void CacheIconForClass(FName InClassName, const FSlateBrush* InSlateBrush) = 0;

	/** Should the scene outliner accept a request to rename a item of the tree */
	virtual bool CanExecuteRenameRequest(const ISceneOutlinerTreeItem& ItemPtr) const = 0;

	/** 
	 * Add a filter to the scene outliner 
	 * @param Filter The filter to apply to the scene outliner
	 * @return The index of the filter.
	 */
	virtual int32 AddFilter(const TSharedRef<FSceneOutlinerFilter>& Filter) = 0;

	/** 
	 * Add a filter to the scene outliner's filter bar
	 * @param Filter The filter to add
	 */
	virtual void AddFilterToFilterBar(const TSharedRef<FFilterBase<SceneOutliner::FilterBarType>>& InFilter) = 0;

	/** 
	 * Disable all filters currently active in the filter bar
	 * @param bRemove if true, the filter will also be unpinned and removed from the filter bar
	 */
	virtual void DisableAllFilterBarFilters(bool bRemove) = 0;

	/** 
	 * Remove a filter from the scene outliner
	 * @param Filter The Filter to remove
	 * @return True if the filter was removed.
	 */
	virtual bool RemoveFilter(const TSharedRef<FSceneOutlinerFilter>& Filter) = 0;

	/** 
	 * Add an interactive filter to the scene outliner 
	 * @param Filter The filter used to determine if scene outliner items are interactive.
	 * @return The index of the filter.
	 */
	virtual int32 AddInteractiveFilter(const TSharedRef<FSceneOutlinerFilter>& Filter) = 0;

	/** 
	 * Remove an interactive  filter from the scene outliner
	 * @param Filter The Filter to remove
	 * @return True if the filter was removed.
	 */
	virtual bool RemoveInteractiveFilter(const TSharedRef<FSceneOutlinerFilter>& Filter) = 0;

	/** 
	 * Retrieve the filter at the specified index
	 * @param Index The index of the filter to retrive
	 * @return A valid poiter to a filter if the index was valid
	 */
	virtual TSharedPtr<FSceneOutlinerFilter> GetFilterAtIndex(int32 Index) = 0;

	/** Get number of filters applied to the scene outliner */
	virtual int32 GetFilterCount() const = 0;

	/**
	 * Add or replace a column of the scene outliner
	 * Note: The column id must match the id of the column returned by the factory
	 * @param ColumnId The id of the column to add
	 * @param ColumInfo The struct that contains the information on how to present and retrieve the column
	 */
	virtual void AddColumn(FName ColumnId, const FSceneOutlinerColumnInfo& ColumnInfo) = 0;

	/**
	 * Remove a column of the scene outliner
	 * @param ColumnId The name of the column to remove
	 */
	virtual void RemoveColumn(FName ColumnId) = 0;

	/** Return the name/Id of the columns of the scene outliner */
	virtual TArray<FName> GetColumnIds() const = 0;

	/** Return the sorting mode for the specified ColumnId */
	virtual EColumnSortMode::Type GetColumnSortMode( const FName ColumnId ) const = 0;

	virtual uint32 GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const = 0;

	/** Request that the tree be sorted at a convenient time */
	virtual void RequestSort() = 0;

	/** Returns true if edit delete can be executed */
	virtual bool Delete_CanExecute() = 0;

	/** Returns true if edit rename can be executed */
	virtual bool Rename_CanExecute() = 0;

	/** Executes rename. */
	virtual void Rename_Execute() = 0;

	/** Returns true if edit cut can be executed */
	virtual bool Cut_CanExecute() = 0;

	/** Returns true if edit copy can be executed */
	virtual bool Copy_CanExecute() = 0;

	/** Returns true if edit paste can be executed */
	virtual bool Paste_CanExecute() = 0;

	/** Can the scene outliner rows generated on drag event */
	virtual bool CanSupportDragAndDrop() const = 0;

	/** Set the item selection of the outliner based on a selector function. Any items which return true will be added */
	virtual void SetSelection(const TFunctionRef<bool(ISceneOutlinerTreeItem&)> Selector) = 0;

	/** Pins an item list in the outliner. */
	virtual void PinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) = 0;

	/**  Unpins an item list in the outliner. */
	virtual void UnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) = 0;
	
	/** Returns true if any of the items can be pinned. */
	virtual bool CanPinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const = 0;

	/** Returns true if any of the items can be unpinned. */
	virtual bool CanUnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const = 0;

	/** Pin selected items */
	virtual void PinSelectedItems() = 0;

	/** Unpins selected items */
	virtual void UnpinSelectedItems() = 0;

	/** Returns true if any of the selected items can be pinned */
	virtual bool CanPinSelectedItems() const = 0;

	/** Returns true if any of the selected items can be unpinned */
	virtual bool CanUnpinSelectedItems() const = 0;

	/** Scrolls the outliner to the selected item(s). If more are selected, the chosen item is undeterministic. */
	virtual void FrameSelectedItems() = 0;

	/** Scrolls the outliner to the specified item if it's found. No action is done otherwise. */
	virtual void FrameItem(const FSceneOutlinerTreeItemID& Item) = 0;

	/** Set whether a particular item in the outliner is expanded or not. */
	virtual void SetItemExpansion(const FSceneOutlinerTreeItemPtr& Item, bool bIsExpanded) {}

	/** Get the active SceneOutlinerMode */
	const ISceneOutlinerMode* GetMode() const { return Mode; }

	/** Get the associated source control object for the specified item. */
	virtual TSharedPtr<FSceneOutlinerTreeItemSCC> GetItemSourceControl(const FSceneOutlinerTreeItemPtr& InItem) = 0;

	/** Check if a filter with the given name exists and is active in the filter bar for this Outliner (if this Outliner has a filter bar). */
	virtual bool IsFilterActive(const FString& FilterName) const = 0;

	/** Retrieve an ISceneOutlinerTreeItem by its ID if it exists in the tree */
	virtual FSceneOutlinerTreeItemPtr GetTreeItem(FSceneOutlinerTreeItemID, bool bIncludePending = false) = 0;

	/** Set a delay (in seconds) before the the next requested UI refresh executes. Note: This only applies until the next refresh that executes
	 * and any subsequent ones will be immediate. */
	virtual void SetNextUIRefreshDelay(float InDelay) = 0;

	/** Get the unqiue identifier of this Outliner (if specified, NAME_None otherwise) */
	virtual FName GetOutlinerIdentifier() const = 0;
protected:
	ISceneOutlinerMode* Mode;
};
