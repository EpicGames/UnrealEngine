// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneOutlinerFwd.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"

template<typename ItemType> class STableRow;

/**
 *	Interface for a scene outliner column
 */
class ISceneOutlinerColumn : public TSharedFromThis< ISceneOutlinerColumn >
{
public:
	virtual ~ISceneOutlinerColumn() = default;

	virtual FName GetColumnID() = 0;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() = 0;

	virtual const TSharedRef< SWidget > ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) = 0;

	virtual void Tick(double InCurrentTime, float InDeltaTime) {}

public:
	/** Optionally overridden interface methods */
	virtual void PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const {}

	virtual bool SupportsSorting() const { return false; }

	virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const {}

	// Called when a sort is first requested in the Outliner
	virtual void OnSortRequested(const EColumnSortPriority::Type SortPriority, const EColumnSortMode::Type InSortMode) {}

	// Checks whether the column is still processing the sort or if it is ready to start sorting items
	virtual bool IsSortReady() { return true; }
};

