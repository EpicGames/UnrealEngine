// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SceneOutlinerStandaloneTypes.h"
#include "SceneOutlinerFwd.h"
#include "ISceneOutlinerTreeItem.h"

class ISceneOutlinerHierarchy : public TSharedFromThis<ISceneOutlinerHierarchy>
{
public:
	ISceneOutlinerHierarchy(ISceneOutlinerMode* InMode) : Mode(InMode) {}
	virtual ~ISceneOutlinerHierarchy() { HierarchyChangedEvent.Clear(); }
			
	/** Create a linearization of all applicable items in the hierarchy */
	virtual void CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const = 0;
	/** Create a linearization of all direct and indirect children of a given item in the hierarchy */
	virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const = 0;
	/** Find or optionally create a parent item for a given tree item */
	virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) = 0;

	DECLARE_EVENT_OneParam(ISceneOutlinerHierarchy, FHierarchyChangedEvent, FSceneOutlinerHierarchyChangedData)
	FHierarchyChangedEvent& OnHierarchyChanged() { return HierarchyChangedEvent; }

protected:
	ISceneOutlinerMode* Mode;
	FHierarchyChangedEvent HierarchyChangedEvent;
};
