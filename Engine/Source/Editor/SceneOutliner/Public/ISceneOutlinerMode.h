// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SceneOutlinerFwd.h"
#include "ISceneOutlinerTreeItem.h"
#include "SceneOutlinerFilters.h"
#include "SSceneOutliner.h"
#include "SceneOutlinerDragDrop.h"
#include "Folder.h"

class FUICommandList;

class ISceneOutlinerMode
{
public:
	ISceneOutlinerMode(SSceneOutliner* InSceneOutliner) : SceneOutliner(InSceneOutliner) {}

	virtual ~ISceneOutlinerMode() = default;

	/** Rebuild all mode data */
	virtual void Rebuild() = 0;

	/** 
		* Attempt to create a tree item for a given data type.
		* @param bForce should the item be created regardless if it passes the filters or not
		* @return returns a shared pointer to the new item if it was created
				may be invalid if the item was not created
		*/
	template <typename TreeItemType, typename TreeItemData>
	FSceneOutlinerTreeItemPtr CreateItemFor(const TreeItemData& Data, bool bForce = false)
	{
		return SceneOutliner->CreateItemFor<TreeItemType>(Data,
			[this](const ISceneOutlinerTreeItem& Item) { OnItemPassesFilters(Item); }, bForce);
	}

	/** Fill the view button menu with content */
	virtual void CreateViewContent(FMenuBuilder& MenuBuilder) {}

	/** Allows extending SceneOutliner View Button MenuBuilder (allows using ExtensionHooks which isn't possible with CreateViewContent override) */
	virtual void InitializeViewMenuExtender(TSharedPtr<FExtender> Extender) { }

	/** Returns a factory to create a Folder Picker mode which matches this mode */
	virtual FCreateSceneOutlinerMode CreateFolderPickerMode(const FFolder::FRootObject& InRootObject = FFolder::GetInvalidRootObject()) const { return FCreateSceneOutlinerMode(); }
public:
	/** Construct a new Drag and drop operation */
	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const { return nullptr; }

	/**
		* Populates a drag/drop operation if the mode supports that type of operation.
		* @param OutPayload:	The payload data will be stored here after parsing
		* @return				Returns true if the operation is supported by the mode
		*/
	virtual bool ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const { return false; /* Default does not support drag/drop */ }

	/** Test whether the specified payload can be dropped onto this tree item */
	virtual FSceneOutlinerDragValidationInfo ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const { return FSceneOutlinerDragValidationInfo::Invalid(); }

	/** Called when a payload is dropped onto a target */
	virtual void OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const {}

	/** Called when a payload is dragged over an item */
	virtual FReply OnDragOverItem(const FDragDropEvent& Event, const ISceneOutlinerTreeItem& Item) const { return FReply::Handled(); }
	
	/** Called when a drag-over/enter/leave warrants updating the DragDrop decorator, (ValidationInfo == nullptr, indicates resetting the decorator) */ 
	virtual bool UpdateOperationDecorator(const FDragDropEvent& Event, const FSceneOutlinerDragValidationInfo* ValidationInfo) const { return false; }
public:
	/* Events */

	/** Called by the outliner when a new item is added to the outliner tree */
	virtual void OnItemAdded(FSceneOutlinerTreeItemPtr Item) {}
	/** Called by the outliner when an item is removed from the outliner tree */
	virtual void OnItemRemoved(FSceneOutlinerTreeItemPtr Item) {}
	/** Called by the outliner when an item is selected in the tree. Used to partially synchronize selection */
	virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) {}
	/** Called by the outliner when a tree item is double clicked */
	virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) {}
	/** Called by the outliner when a tree item is single-clicked */
	virtual void OnItemClicked(FSceneOutlinerTreeItemPtr Item) {}
	/** Called by the outliner when search box text is changed */
	virtual void OnFilterTextChanged(const FText& InFilterText) {}
	/** Called by the outliner when search box text is committed (by pressing enter) */
	virtual void OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType) {}
	/** Called by the outliner when an item passes filters but isn't yet added to the tree */
	virtual void OnItemPassesFilters(const ISceneOutlinerTreeItem& Item) {}
	/** Called by the outliner when a key is pressed */
	virtual FReply OnKeyDown(const FKeyEvent& InKeyEvent) { return FReply::Unhandled(); }

	/** Returns a context menu widget based on the current selection */
	virtual TSharedPtr<SWidget> CreateContextMenu() { return TSharedPtr<SWidget>(); }
	/** Check if an item can be renamed */
	virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const { return false; }
	/** Whether the toolbar can be customized. */
	virtual bool CanCustomizeToolbar() const { return false; }
	/** Add mode-specific controls to the toolbar. Called when CanCustomizeToolbar() returns true. */
	virtual void CustomAddToToolbar(TSharedPtr<SHorizontalBox> Toolbar) {}
	/** Check if an item is interactive */
	virtual bool CanInteract(const ISceneOutlinerTreeItem& Item) const { return true; }
	/** Check if we are allowed to repopulate */
	virtual bool CanPopulate() const { return true; }

	/** Synchronize the mode specific selection with the tree view */
	virtual void SynchronizeSelection() {}
	/** Trigger a duplication of selected items */
	virtual void OnDuplicateSelected() {}

	/** Returns the root object of this outliner mode */
	virtual FFolder::FRootObject GetRootObject() const { return FFolder::GetInvalidRootObject(); }
	/** Returns the target root object for a paste operation */
	virtual FFolder::FRootObject GetPasteTargetRootObject() const { return FFolder::GetInvalidRootObject(); }
public:
	/** Get the current selection mode */
	virtual ESelectionMode::Type GetSelectionMode() const { return ESelectionMode::Single; }
	/** Get the sort priority of an item */
	virtual int32 GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const { return 0; }
	/** Does the mode support keyboard focus */
	virtual bool SupportsKeyboardFocus() const { return false; }
	/** Get the current status text */
	virtual FText GetStatusText() const { return FText(); }
	/** Get the color of the status text */
	virtual FSlateColor GetStatusTextColor() const { return FSlateColor(); }
	/** Should folders be shown in the outliner */
	virtual bool ShouldShowFolders() const { return false; }
	/** Should the new folder button be displayed by the outliner */
	virtual bool SupportsCreateNewFolder() const { return false; }
	/** Is the mode interactive */
	virtual bool IsInteractive() const { return false; }
	/** Does the mode want to display a status bar */
	virtual bool ShowStatusBar() const { return false; }
	/** Does the mode want to display a view button */
	virtual bool ShowViewButton() const { return false; }
	/** Does the mode allow the user to enable and disable filters through a menu */
	virtual bool ShowFilterOptions() const { return false; }

	/** Can the current selection be deleted */
	virtual bool CanDelete() const { return false; }
	/** Can the current selection be renamed */
	virtual bool CanRename() const { return false; }
	/** Can the current selection be cut */
	virtual bool CanCut() const { return false; }
	/** Can the current selection be copied */
	virtual bool CanCopy() const { return false; }
	/** Can the current selection be pasted */
	virtual bool CanPaste() const { return false; }

	/** Does the mode support drag and drop */
	virtual bool CanSupportDragAndDrop() const { return false; }

	/** Does the mode reports errors */
	virtual bool HasErrors() const { return false; }

	/** Return the errors text */
	virtual FText GetErrorsText() const { return FText(); }

	/** Repair errors*/
	virtual void RepairErrors() const {}

	/** Bind commands for this mode to the given CommandList */
	virtual void BindCommands(const TSharedRef<FUICommandList>& OutCommandList) {}

public:
	/* Folder management */
		
	/** Creates a new folder item at the root with a valid name*/
	virtual FFolder CreateNewFolder() { return FFolder::GetInvalidFolder(); }
	/** Returns a unique folder path for a specific parent with a given leaf name (without creating it) */
	virtual FFolder GetFolder(const FFolder& ParentPath, const FName& LeafName) { return FFolder::GetInvalidFolder(); }
	/** Create a folder under a specific parent with a given leaf name */
	virtual bool CreateFolder(const FFolder& NewFolder) { return false; }
	/** Reparent an item to a given folder path. Returns true if the operation is sucessful */
	virtual bool ReparentItemToFolder(const FFolder& FolderPath, const FSceneOutlinerTreeItemPtr& Item) { return false; }
	/** Select all descendants of a folder. Optionally select only immediate descendants. */
	virtual void SelectFoldersDescendants(const TArray<FFolderTreeItem*>& FolderItems, bool bSelectImmediateChildrenOnly) {}
	/** Returns true if this mode overrides the default folder double click behavior (expanding the subtree) */
	virtual bool HasCustomFolderDoubleClick() const { return false; }
public:
	/** Pins an item list in the outliner */
	virtual void PinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) {}
	/** Unpins an item list in the outliner */
	virtual void UnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) {}
	/** Returns true if any of the items can be pinned. */
	virtual bool CanPinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const { return false; }
	/** Returns true if any of the items can be unpinned. */
	virtual bool CanUnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const { return false; }

	/** Function called by the Outliner Filter Bar to compare an item with Type Filters*/
	virtual bool CompareItemWithClassName(SceneOutliner::FilterBarType InItem, const TSet<FTopLevelAssetPath>&) const { return false; };

	/** Function called at the start of the Outliner's tick to allow the mode to perform operations */
	virtual void Tick() {}
public:
	/* Getters */

	ISceneOutlinerHierarchy* GetHierarchy() { return Hierarchy.Get(); }
	TMap<FName, FSceneOutlinerFilterInfo>& GetFilterInfos() { return FilterInfoMap; }
protected:
	virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() = 0;
protected:
	TMap<FName, FSceneOutlinerFilterInfo> FilterInfoMap;
	TUniquePtr<ISceneOutlinerHierarchy> Hierarchy;

	SSceneOutliner* SceneOutliner;
private:
	ISceneOutlinerMode() = delete;
	ISceneOutlinerMode(const ISceneOutliner&) = delete;
	ISceneOutlinerMode& operator=(const ISceneOutliner&) = delete;
};
