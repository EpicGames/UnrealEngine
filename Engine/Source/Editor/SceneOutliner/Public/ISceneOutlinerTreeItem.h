// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "SceneOutlinerStandaloneTypes.h"

#define UE_API SCENEOUTLINER_API

class ISceneOutliner;
class UToolMenu;

/** Base tree item interface  */
struct ISceneOutlinerTreeItem : TSharedFromThis<ISceneOutlinerTreeItem>
{
	/** Friendship required for access to various internals */
	friend SSceneOutliner;

	/* Flags structure */
	struct FlagsType
	{
		/** Whether this item is expanded or not */
		bool bIsExpanded : 1;

		/* true if this item is filtered out */
		bool bIsFilteredOut : 1;

		/* true if this item can be interacted with as per the current outliner filters */
		bool bInteractive : 1;

		/** true if this item's children need to be sorted */
		bool bChildrenRequireSort : 1;

		/** Default constructor */		
		FlagsType() : bIsExpanded(1), bIsFilteredOut(0), bInteractive(1), bChildrenRequireSort(1) {}
	};

public:
		
	/** Flags for this item */
	FlagsType Flags;

	/** Weak Outliner pointer */
	TWeakPtr<ISceneOutliner> WeakSceneOutliner;

	/** Delegate for hooking up an inline editable text block to be notified that a rename is requested. */
	DECLARE_DELEGATE( FOnRenameRequest );

	/** Broadcasts whenever a rename is requested */
	FOnRenameRequest RenameRequestEvent;

protected:

	/** Default constructor */
	ISceneOutlinerTreeItem(FSceneOutlinerTreeItemType InType) : Parent(nullptr), TreeType(InType) {}
	virtual ~ISceneOutlinerTreeItem() = default;

	/** This item's parent, if any. */
	mutable TWeakPtr<ISceneOutlinerTreeItem> Parent;

	/** Array of children contained underneath this item */
	mutable TSet<TWeakPtr<ISceneOutlinerTreeItem>> Children;

	/** Static type identifier for the base class tree item */
	static UE_API const FSceneOutlinerTreeItemType Type;

	/** Tree item type identifier */
	FSceneOutlinerTreeItemType TreeType;
public:

	/** Get this item's parent. Can be nullptr. */
	FSceneOutlinerTreeItemPtr GetParent() const
	{
		return Parent.Pin();
	}

	/** Add a child to this item */
	void AddChild(FSceneOutlinerTreeItemRef Child)
	{
		check(!Children.Contains(Child));
		Child->Parent = AsShared();
		Children.Add(MoveTemp(Child));
	}

	/** Remove a child from this item */
	void RemoveChild(const FSceneOutlinerTreeItemRef& Child)
	{
		if (Children.Remove(Child))
		{
			Child->Parent = nullptr;	
		}
	}

	/** Get this item's children, if any. */
	FORCEINLINE const TSet<TWeakPtr<ISceneOutlinerTreeItem>>& GetChildren() const
	{
		return Children;
	}

	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const ISceneOutlinerTreeItem& Item);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const ISceneOutlinerTreeItem& Item);

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(*this);
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(*this);
	}
public:
	/** Attempt to cast this item to another type if it is of that type. Returns nullptr if it fails */
	template <typename T>
	T* CastTo()
	{
		return TreeType.IsA(T::Type) ? StaticCast<T*>(this) : nullptr;
	}
	/** Attempt to cast this item to another type if it is of that type. Returns nullptr if it fails */
	template <typename T>
	const T* CastTo() const
	{
		return TreeType.IsA(T::Type) ? StaticCast<const T*>(this) : nullptr;
	}
	/** Returns true if this item is of the specified type */
	template <typename T>
	bool IsA() const
	{
		return TreeType.IsA(T::Type);
	}
	
	bool IsA(const FSceneOutlinerTreeItemType& InType) const
	{
		return TreeType.IsA(InType);
	}
	
	const FSceneOutlinerTreeItemType& GetType() const { return TreeType; }

	/** Returns true if the data the item references is valid */
	virtual bool IsValid() const = 0;

public:

	/** Get the ID that represents this tree item. Used to reference this item in a map */
	virtual FSceneOutlinerTreeItemID GetID() const = 0;

	/** Get the optional context object root of a tree item. */
	virtual FFolder::FRootObject GetRootObject() const { return FFolder::GetInvalidRootObject(); }

	/** Get the raw string to display for this tree item - used for sorting */
	virtual FString GetDisplayString() const = 0;

	/** Check whether it should be possible to interact with this tree item */
	virtual bool CanInteract() const = 0;
		
	/** Called when this item is expanded or collapsed */
	virtual void OnExpansionChanged() {};

	/** Generate the label widget for this item */
	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) { return SNullWidget::NullWidget; }

	/** Generate a context menu for this item. Only called if *only* this item is selected. */
	virtual void GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner) {}

	/** Called when this items visibility changed */
	virtual void OnVisibilityChanged(const bool bNewVisibility) {}

	/** Returns true if this item should show its visibility state */
	virtual bool ShouldShowVisibilityState() const { return true; }

	/** Returns true if this item can set its own visibility */
	virtual bool HasVisibilityInfo() const { return false; }

	/** Query this items visibility state. Only called if the item type has visibility info */
	virtual bool GetVisibility() const { return false; }

	/** Returns true if this item should show pinned state */
	virtual bool ShouldShowPinnedState() const { return false; }

	/** Returns true if this item can set its own pinned state */
	virtual bool HasPinnedStateInfo() const { return false; }

	/** Query the pinned staet of this item. Only called if the item has its own pinned state info */
	virtual bool GetPinnedState() const { return false; }

	/** Called when this item's label has changed */
	virtual void OnLabelChanged() {}

	/** Whether the item should be removed when the last child has been removed. */
	virtual bool ShouldRemoveOnceLastChildRemoved() const { check(GetChildren().IsEmpty()); return Flags.bIsFilteredOut; }

	/** Returns (optionally) the name of the UPackage this item represents/is-part-of */
	virtual FString GetPackageName() const { return FString(); }
};

#undef UE_API
