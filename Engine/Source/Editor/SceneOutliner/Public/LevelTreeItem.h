// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"

#define UE_API SCENEOUTLINER_API

class ULevel;

/** A tree item that represents a level in the world */
struct FLevelTreeItem : ISceneOutlinerTreeItem
{
public:
	/** Static type identifier for this tree item class */
	static UE_API const FSceneOutlinerTreeItemType Type;

	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const ULevel*);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const ULevel*);

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(Level.Get());
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(Level.Get());
	}

	/** The level this tree item is associated with. */
	mutable TWeakObjectPtr<ULevel> Level;

	/** Constant identifier for this tree item */
	const FObjectKey ID;

	/** Construct this item from a level */
	UE_API FLevelTreeItem(ULevel* InLevel);

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return Level.IsValid(); }
	virtual FSceneOutlinerTreeItemID GetID() const override { return ID; }
	UE_API virtual FString GetDisplayString() const override;
	UE_API virtual bool CanInteract() const override;
	UE_API virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	UE_API virtual void GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner) override;
	/* End ISceneOutlinerTreeItem Implementation */
};

#undef UE_API
