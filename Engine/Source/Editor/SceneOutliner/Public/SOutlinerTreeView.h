// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SceneOutlinerFwd.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Animation/SlateSprings.h"

template<>
struct TWidgetTypeTraits<class SSceneOutlinerTreeView>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SSceneOutlinerTreeView : public STreeView< FSceneOutlinerTreeItemPtr >
{
public:
		
	/** Construct this widget */
	void Construct(const FArguments& InArgs, TSharedRef<SSceneOutliner> Owner);

	void FlashHighlightOnItem( FSceneOutlinerTreeItemPtr FlashHighlightOnItem );

	const TWeakPtr<SSceneOutliner>& GetOutlinerPtr() { return SceneOutlinerWeak; }
protected:

	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;

	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

private:
	virtual void Private_UpdateParentHighlights() override;

	/** Weak reference to the outliner widget that owns this list */
	TWeakPtr<SSceneOutliner> SceneOutlinerWeak;
};

template<>
struct TWidgetTypeTraits<class SSceneOutlinerTreeRow>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** Widget that represents a row in the outliner's tree control.  Generates widgets for each column on demand. */
class SSceneOutlinerTreeRow
	: public SMultiColumnTableRow< FSceneOutlinerTreeItemPtr >
{

public:

	SLATE_BEGIN_ARGS( SSceneOutlinerTreeRow ) {}

		/** The list item for this row */
		SLATE_ARGUMENT( FSceneOutlinerTreeItemPtr, Item )

	SLATE_END_ARGS()


	/** Construct function for this widget */
	void Construct( const FArguments& InArgs, const TSharedRef<SSceneOutlinerTreeView>& OutlinerTreeView, TSharedRef<SSceneOutliner> SceneOutliner );

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

	void FlashHighlight();

protected:
	virtual bool ComputeVolatility() const override;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;

	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	bool IsRowEnabled() const;

private:

	/** Weak reference to the outliner widget that owns our list */
	TWeakPtr< SSceneOutliner > SceneOutlinerWeak;

	/** The item associated with this row of data */
	TWeakPtr<ISceneOutlinerTreeItem> Item;

protected:

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

private:
	/** How many pixels to extend the highlight rectangle's left side horizontally */
	static const float HighlightRectLeftOffset;

	/** How many pixels to extend the highlight rectangle's right side horizontally */
	static const float HighlightRectRightOffset;

	/** How quickly the highlight 'targeting' rectangle will slide around.  Larger is faster. */
	static const float HighlightTargetSpringConstant;

	/** Duration of animation highlight target effects */
	static const float HighlightTargetEffectDuration;

	/** Opacity of the highlight target effect overlay */
	static const float HighlightTargetOpacity;

	/** How large the highlight target effect will be when highlighting, as a scalar percentage of font height */
	static const float LabelChangedAnimOffsetPercent;

	/** Highlight "targeting" visual effect left position */
	FFloatSpring1D HighlightTargetLeftSpring;

	/** Highlight "targeting" visual effect right position */
	FFloatSpring1D HighlightTargetRightSpring;

	/** Last time that the user had a major interaction with the highlight */
	double LastHighlightInteractionTime;
};

template<>
struct TWidgetTypeTraits<class SSceneOutlinerPinnedTreeRow>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** Widget that represents a pinned row in the outliner's tree control.*/
class SSceneOutlinerPinnedTreeRow
	: public SMultiColumnTableRow< FSceneOutlinerTreeItemPtr >
{

public:

	SLATE_BEGIN_ARGS(SSceneOutlinerPinnedTreeRow) {}

		/** The list item for this row */
		SLATE_ARGUMENT(FSceneOutlinerTreeItemPtr, Item)

	SLATE_END_ARGS()


	/** Construct function for this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<SSceneOutlinerTreeView>& OutlinerTreeView, TSharedRef<SSceneOutliner> SceneOutliner);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

protected:

	virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override
	{
		STableRow<FSceneOutlinerTreeItemPtr>::Content = InContent;

		// We don't want the expander by default.
		this->ChildSlot
			.Padding(InPadding)
			[
				InContent
			];
	}

	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

private:

	/** Weak reference to the outliner widget that owns our list */
	TWeakPtr< SSceneOutliner > SceneOutlinerWeak;

	/** Weak reference to the tree view that contains our list */
	TWeakPtr< SSceneOutlinerTreeView > OutlinerTreeViewWeak;

	/** The item associated with this row of data */
	TWeakPtr<ISceneOutlinerTreeItem> Item;
};