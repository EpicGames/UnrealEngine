// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Views/ITypedTableView.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/TextFilter.h"
#include "SlateFwd.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Delegates/DelegateCombinations.h"
#include "Framework/Views/ITypedTableView.h"
#include "Types/SlateEnums.h"
#include "Folder.h"

#include "ISceneOutliner.h"
#include "SceneOutlinerFwd.h"

#include "SOutlinerTreeView.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerStandaloneTypes.h"

#include "ISceneOutlinerHierarchy.h"
#include "SceneOutlinerDragDrop.h"

#include "SceneOutlinerSCCHandler.h"
#include "SceneOutlinerTreeItemSCC.h"

#define UE_API SCENEOUTLINER_API

class FMenuBuilder;
class UToolMenu;
class ISceneOutlinerColumn;
class SComboButton;
class ULevel;
struct FToolMenuSection;
template<typename FilterType> class SFilterBar;
template<typename ItemType> class STreeView;
class SFilterSearchBox;
struct FCustomTextFilterData;
template<typename FilterType> class ICustomTextFilter;

SCENEOUTLINER_API DECLARE_LOG_CATEGORY_EXTERN(LogSceneOutliner, Log, All);

/**
 * Scene Outliner definition
 * Note the Scene Outliner is also called the World Outliner
 */
namespace SceneOutliner
{
	DECLARE_EVENT_OneParam(SSceneOutliner, FTreeItemPtrEvent, FSceneOutlinerTreeItemPtr);

	DECLARE_EVENT_TwoParams(SSceneOutliner, FOnItemSelectionChanged, FSceneOutlinerTreeItemPtr, ESelectInfo::Type);

	typedef TTextFilter< const ISceneOutlinerTreeItem& > TreeItemTextFilter;

	/** Structure that defines an operation that should be applied to the tree */
	struct FPendingTreeOperation
	{
		enum EType { Added, Removed, Moved };
		FPendingTreeOperation(EType InType, TSharedRef<ISceneOutlinerTreeItem> InItem) : Type(InType), Item(InItem) { }

		/** The type of operation that is to be applied */
		EType Type;

		/** The tree item to which this operation relates */
		FSceneOutlinerTreeItemRef Item;
	};

	/** Set of actions to apply to new tree items */
	namespace ENewItemAction
	{
		enum Type : uint8
		{
			/** Do nothing when it is created */
			None = 0,
			/** Select the item when it is created */
			Select = 1 << 0,
			/** Scroll the item into view when it is created */
			ScrollIntoView = 1 << 1,
			/** Interactively rename the item when it is created (implies the above) */
			Rename = 1 << 2,
		};
	}

	namespace ExtensionHooks
	{
		static FName Hierarchy(TEXT("Hierarchy"));
		static FName Show(TEXT("Show"));
	}
}

/**
	 * Stores a set of selected items with parsing functions for the scene outliner
	 */
struct FSceneOutlinerItemSelection
{
	/** Set of selected items */
	mutable TArray<TWeakPtr<ISceneOutlinerTreeItem>> SelectedItems;

	FSceneOutlinerItemSelection() {}

	FSceneOutlinerItemSelection(const TArray<FSceneOutlinerTreeItemPtr>& InSelectedItems)
		: SelectedItems(InSelectedItems) {}

	FSceneOutlinerItemSelection(SSceneOutlinerTreeView& Tree)
		: FSceneOutlinerItemSelection(Tree.GetSelectedItems()) {}

	/** Returns true if the selection has an item of a specified type */
	template <typename TreeType>
	bool Has() const
	{
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : SelectedItems)
		{
			if (const auto ItemPtr = Item.Pin())
			{
				if (ItemPtr->IsA<TreeType>())
				{
					return true;
				}
			}
		}
		return false;
	}

	/** Returns the total number of items in the selection */
	uint32 Num() const
	{
		return SelectedItems.Num();
	}

	/** Returns the number of items of the specified types in the selection */
	template <typename ...TreeTypes>
	uint32 Num() const
	{
		uint32 Result = 0;
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : SelectedItems)
		{
			if (const auto ItemPtr = Item.Pin())
			{
				if ((ItemPtr->IsA<TreeTypes>() || ...))
				{
					++Result;
				}
			}
		}
		return Result;
	}

	/** Add a new item to the selection */
	void Add(FSceneOutlinerTreeItemPtr NewItem)
	{
		SelectedItems.Add(NewItem);
	}

	/** Get all items of a specified type */
	template <typename TreeType>
	void Get(TArray<TreeType*>& OutArray) const
	{
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : SelectedItems)
		{
			if (const auto ItemPtr = Item.Pin())
			{
				if (TreeType* CastedItem = ItemPtr->CastTo<TreeType>())
				{
					OutArray.Add(CastedItem);
				}
			}
		}
	}

	void Get(TArray<FSceneOutlinerTreeItemPtr>& OutArray) const
	{
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : SelectedItems)
		{
			if (const auto ItemPtr = Item.Pin())
			{
				OutArray.Add(ItemPtr);
			}
		}
	}

	/** Apply a function to each item of a specified type */
	template <typename TreeType>
	void ForEachItem(TFunctionRef<void(TreeType&)> Func) const
	{
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : SelectedItems)
		{
			if (const auto ItemPtr = Item.Pin())
			{
				if (TreeType* CastedItem = ItemPtr->CastTo<TreeType>())
				{
					Func(*CastedItem);
				}
			}
		}
	}

	/** Apply a function to every item in the selection regardless of type. */
	void ForEachItem(TFunctionRef<void(FSceneOutlinerTreeItemPtr&)> Func) const
	{
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : SelectedItems)
		{
			if (auto ItemPtr = Item.Pin())
			{
				Func(ItemPtr);
			}
		}
	}

	/** Use a selector to retrieve a specific data type from items in the selection. Will only add an item's data if the selector returns true for that item. */
	template <typename DataType>
	TArray<DataType> GetData(TFunctionRef<bool(const TWeakPtr<ISceneOutlinerTreeItem>&, DataType&)> Selector) const
	{
		TArray<DataType> Result;
		for (TWeakPtr<ISceneOutlinerTreeItem>& Item : SelectedItems)
		{
			DataType Data;
			if (Selector(Item, Data))
			{
				Result.Add(Data);
			}
		}
		return Result;
	}
};

/**
	* Scene Outliner widget
	*/
class SSceneOutliner : public ISceneOutliner, public FEditorUndoClient, public FGCObject
{

public:

	SLATE_BEGIN_ARGS(SSceneOutliner)
	{}
	SLATE_END_ARGS()

	/**
	* Construct this widget.  Called by the SNew() Slate macro.
	*
	* @param	InArgs		Declaration used by the SNew() macro to construct this widget
	* @param	InitOptions	Programmer-driven initialization options for this widget
	*/
	UE_API void Construct(const FArguments& InArgs, const FSceneOutlinerInitializationOptions& InitOptions);

	/** Default constructor - initializes data that is shared between all tree items */
	SSceneOutliner() 
		: SharedData(MakeShareable(new FSharedSceneOutlinerData)) 
		, Filters(MakeShareable(new FSceneOutlinerFilters))
		, InteractiveFilters(MakeShareable(new FSceneOutlinerFilters))
	{}

	/** SSceneOutliner destructor */
	UE_API virtual ~SSceneOutliner();

	/** SWidget interface */
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual bool SupportsKeyboardFocus() const override;
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Sends a requests to the Scene Outliner to refresh itself the next chance it gets */
	UE_API virtual void Refresh() override;

	UE_API void RefreshSelection();

	//~ Begin FEditorUndoClient Interface
	UE_API virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

	/** @return Returns the common data for this outliner */
	virtual const FSharedSceneOutlinerData& GetSharedData() const override
	{
		return *SharedData;
	}

	/** Get a const reference to the actual tree hierarchy */
	virtual const STreeView<FSceneOutlinerTreeItemPtr>& GetTree() const override
	{
		return *OutlinerTreeView;
	}

	virtual const TSharedPtr< SSceneOutlinerTreeView>& GetTreeView() const
	{
		return OutlinerTreeView;
	}

	/** @return Returns a string to use for highlighting results in the outliner list */
	UE_API virtual TAttribute<FText> GetFilterHighlightText() const override;

	/** @return Returns an Event to use for highlighting changes in the outliner list */
	UE_API SceneOutliner::TreeItemTextFilter::FChangedEvent& OnTextFilterChanged() const;

	/** Set the keyboard focus to the outliner */
	UE_API virtual void SetKeyboardFocus() override;

	/** Gets the cached icon for this class name */
	UE_API virtual const FSlateBrush* GetCachedIconForClass(FName InClassName) const override;

	/** Sets the cached icon for this class name */
	UE_API virtual void CacheIconForClass(FName InClassName, const FSlateBrush* InSlateBrush) override;

	/** Should the scene outliner accept a request to rename a object */
	UE_API virtual bool CanExecuteRenameRequest(const ISceneOutlinerTreeItem& ItemPtr) const override;

	/**
	 * Add a filter to the scene outliner
	 * @param Filter The filter to apply to the scene outliner
	 * @return The index of the filter.
	 */
	UE_API virtual int32 AddFilter(const TSharedRef<FSceneOutlinerFilter>& Filter) override;

	/** 
	 * Add a filter to the scene outliner's filter bar
	 * @param Filter The filter to add
	 */
	UE_API virtual void AddFilterToFilterBar(const TSharedRef<FFilterBase<SceneOutliner::FilterBarType>>& InFilter) override;

	/** 
	 * Disable all filters currently active in the filter bar
	 * @param bRemove if true, the filter will also be unpinned and removed from the filter bar
	 */
	UE_API virtual void DisableAllFilterBarFilters(bool bRemove) override;
	

	/**
	 * Remove a filter from the scene outliner
	 * @param Filter The Filter to remove
	 * @return True if the filter was removed.
	 */
	UE_API virtual bool RemoveFilter(const TSharedRef<FSceneOutlinerFilter>& Filter) override;

	/**
	 * Add an interactive filter to the scene outliner
	 * @param Filter The filter used to determine if scene outliner items are interactive.
	 * @return The index of the filter.
	 */
	UE_API virtual int32 AddInteractiveFilter(const TSharedRef<FSceneOutlinerFilter>& Filter) override;

	/**
	 * Remove an interactive  filter from the scene outliner
	 * @param Filter The Filter to remove
	 * @return True if the filter was removed.
	 */
	UE_API virtual bool RemoveInteractiveFilter(const TSharedRef<FSceneOutlinerFilter>& Filter) override;

	/**
	 * Retrieve the filter at the specified index
	 * @param Index The index of the filter to retrive
	 * @return A valid poiter to a filter if the index was valid
	 */
	UE_API virtual TSharedPtr<FSceneOutlinerFilter> GetFilterAtIndex(int32 Index) override;

	/** Get number of filters applied to the scene outliner */
	UE_API virtual int32 GetFilterCount() const override;

	/**
	 * Add or replace a column of the scene outliner
	 * Note: The column id must match the id of the column returned by the factory
	 * @param ColumnId The id of the column to add
	 * @param ColumnInfo The struct that contains the information on how to present and retrieve the column
	 */
	UE_API virtual void AddColumn(FName ColumnId, const FSceneOutlinerColumnInfo& ColumnInfo) override;

	/**
	 * Remove a column of the scene outliner
	 * @param ColumnId The name of the column to remove
	 */
	UE_API virtual void RemoveColumn(FName ColumnId) override;

	UE_API void SetColumnVisibility(FName ColumnId, bool bIsVisible);

	/** Return the name/Id of the columns of the scene outliner */
	UE_API virtual TArray<FName> GetColumnIds() const override;

	/** @return Returns the current sort mode of the specified column */
	UE_API virtual EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/** Request that the tree be sorted at a convenient time */
	UE_API virtual void RequestSort();

	/** Returns true if edit delete can be executed */
	UE_API virtual bool Delete_CanExecute();

	/** Returns true if edit rename can be executed */
	UE_API virtual bool Rename_CanExecute();

	/** Executes rename. */
	UE_API virtual void Rename_Execute();

	/** Returns true if edit cut can be executed */
	UE_API virtual bool Cut_CanExecute();

	/** Returns true if edit copy can be executed */
	UE_API virtual bool Copy_CanExecute();

	/** Returns true if edit paste can be executed */
	UE_API virtual bool Paste_CanExecute();

	/** Can the scene outliner rows generated on drag event */
	UE_API virtual bool CanSupportDragAndDrop() const override;

	/** Tells the scene outliner that it should do a full refresh, which will clear the entire tree and rebuild it from scratch. */
	UE_API virtual void FullRefresh() override;

	/** Hook to add custom options to toolbar in a derived class. Defaults to delegating to the mode's CustomAddToToolbar. */
	UE_API virtual void CustomAddToToolbar(TSharedPtr<class SHorizontalBox> Toolbar);

	/** Check if a filter with the given name exists and is active in the filter bar for this Outliner (if this Outliner has a filter bar). */
	UE_API virtual bool IsFilterActive(const FString& FilterName) const override;

	/** Retrieve an ISceneOutlinerTreeItem by its ID if it exists in the tree */
	UE_API virtual FSceneOutlinerTreeItemPtr GetTreeItem(FSceneOutlinerTreeItemID, bool bIncludePending = false) override;
	
	/** Set a delay (in seconds) before the the next requested UI refresh executes. Note: This only applies until the next refresh that executes
	 * and any subsequent ones will be immediate. */
	UE_API virtual void SetNextUIRefreshDelay(float InDelay) override;

public:
	/** Event to react to a user double click on a item */
	SceneOutliner::FTreeItemPtrEvent& GetDoubleClickEvent() { return OnDoubleClickOnTreeEvent; }

	/**
		* Allow the system that use the scene outliner to react when it's selection is changed
		* Note: This event will only be broadcast on a user input.
		*/
	SceneOutliner::FOnItemSelectionChanged& GetOnItemSelectionChanged() { return OnItemSelectionChanged; }

	/** Event that's broadcast after the scene outliner has been re-populated. */
	FSimpleMulticastDelegate& GetOnFullRefreshDone() { return OnFullRefreshDone; }

	/** Set the item selection of the outliner based on a selector function. Any items which return true will be added */
	UE_API virtual void SetSelection(const TFunctionRef<bool(ISceneOutlinerTreeItem&)> Selector) override;

	/** Set the selection status of a set of items in the scene outliner */
	UE_API void SetItemSelection(const TArray<FSceneOutlinerTreeItemPtr>& InItems, bool bSelected, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Set the selection status of a single item in the scene outliner */
	UE_API void SetItemSelection(const FSceneOutlinerTreeItemPtr& InItem, bool bSelected, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Adds a set of items to the current selection */
	UE_API void AddToSelection(const TArray<FSceneOutlinerTreeItemPtr>& InItems, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Remove a set of items from the current selection */
	UE_API void RemoveFromSelection(const TArray<FSceneOutlinerTreeItemPtr>& InItems, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);
	
	/** Remove a single item from the current selection */
	UE_API void RemoveFromSelection(const FSceneOutlinerTreeItemPtr& InItem, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Adds a single item to the current selection */
	UE_API void AddToSelection(const FSceneOutlinerTreeItemPtr& InItem, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/**
		* Returns the list of currently selected tree items
		*/
	virtual TArray<FSceneOutlinerTreeItemPtr> GetSelectedItems() const { return OutlinerTreeView->GetSelectedItems(); }

	/**
		* Returns the currently selected items.
		*/
	virtual FSceneOutlinerItemSelection GetSelection() const { return FSceneOutlinerItemSelection(*OutlinerTreeView); }

	/**
	 * Pins an item list in the outliner.
	 */
	UE_API virtual void PinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) override;

	/**
	 * Unpins an item list in the outliner.
	 */
	UE_API virtual void UnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) override;

	/**
	 * Returns true if any of the items can be pinned.
	 */
	UE_API virtual bool CanPinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const override;

	/**
	 * Returns true if any of the items can be unpinned.
	 */
	UE_API virtual bool CanUnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const override;

	/**
	 * Pin selected items
	 */
	UE_API virtual void PinSelectedItems() override;

	/**
	 * Unpins selected items
	 */
	UE_API virtual void UnpinSelectedItems() override;

	/**
	 * Returns true if any of the selected items can be pinned
	 */
	UE_API virtual bool CanPinSelectedItems() const override;

	/**
	 * Returns true if any of the selected items can be unpinned
	 */
	UE_API virtual bool CanUnpinSelectedItems() const override;
	
	/**
	 * Scrolls the outliner to the selected item(s).
	 * If more are selected, the chosen item is undeterministic.
	 */
	UE_API virtual void FrameSelectedItems() override;

	/**
	 * Scrolls the outliner to the specified item if it's found.
	 * No action is done otherwise.
	 */
	UE_API virtual void FrameItem(const FSceneOutlinerTreeItemID& Item) override;

	/**
	 * Returns the parent tree item for a given item if it exists, nullptr otherwise.
	 */
	UE_API FSceneOutlinerTreeItemPtr FindParent(const ISceneOutlinerTreeItem& InItem) const;

	/**
		* Add a folder to the selection of the scene outliner
		* @param FolderName The name of the folder to add to selection
		*/
	UE_API void AddFolderToSelection(const FName& FolderName);

	/**
		* Remove a folder from the selection of the scene outliner
		* @param FolderName The name of the folder to remove from the selection
		*/
	UE_API void RemoveFolderFromSelection(const FName& FolderName);

	/** Deselect all selected items */
	UE_API void ClearSelection();

	/** Sets the next item to rename */
	void SetPendingRenameItem(const FSceneOutlinerTreeItemPtr& InItem) { PendingRenameItem = InItem; Refresh(); }
	
	/** Get the outliner filter collection */
	TSharedPtr<FSceneOutlinerFilters>& GetFilters() { return Filters; }

	/** Create a drag drop operation */
	UE_API TSharedPtr<FDragDropOperation> CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const;

	/** Parse a drag drop operation into a payload */
	UE_API bool ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const;

	/** Validate a drag drop operation on a drop target */
	UE_API FSceneOutlinerDragValidationInfo ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const;

	/** Called when a payload is dropped onto a target */
	UE_API void OnDropPayload(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const;

	/** Called when a payload is dragged over an item */
	UE_API FReply OnDragOverItem(const FDragDropEvent& Event, const ISceneOutlinerTreeItem& Item) const;
	
	/** Called when a drag-over/enter/leave warrants updating the DragDrop decorator */ 
	UE_API bool UpdateOperationDecorator(const FDragDropEvent& Event, const FSceneOutlinerDragValidationInfo* ValidationInfo) const;	

	/** Used to test if Outliner related selection changes have already been handled */
	bool GetIsReentrant() const { return bIsReentrant; }

	/** Get the unique identifier associated with this outliner */
	virtual FName GetOutlinerIdentifier() const override { return OutlinerIdentifier; }

	/** Toggle if SceneOutliner should show Transient objects */
	void SetShowTransient(bool bInShowTransient) { SharedData.Get()->bShowTransient = bInShowTransient; }

private:
	/** Methods that implement structural modification logic for the tree */

	/** Empty all the tree item containers maintained by this outliner */
	UE_API void EmptyTreeItems();

	/** Apply incremental changes to, or a complete repopulation of the tree  */
	UE_API void Populate();

	/** Repopulates the entire tree */
	UE_API void RepopulateEntireTree();

	/** Adds a single new item to the pending map and creates an add operation for it */
	UE_API void AddPendingItem(FSceneOutlinerTreeItemPtr Item);

	/** Adds a new item and all of its children to the pending items. */
	UE_API void AddPendingItemAndChildren(FSceneOutlinerTreeItemPtr Item);

	/** Attempts to add a pending item to the current tree. Will add any parents if required. */
	UE_API bool AddItemToTree(FSceneOutlinerTreeItemRef InItem);

	/** Add an item to the tree, even if it doesn't match the filter terms. Used to add parent's that would otherwise be filtered out */
	UE_API void AddUnfilteredItemToTree(FSceneOutlinerTreeItemRef Item);

	/** Ensure that the specified item's parent is added to the tree, if applicable */
	UE_API FSceneOutlinerTreeItemPtr EnsureParentForItem(FSceneOutlinerTreeItemRef Item);

	/** Remove the specified item from the tree */
	UE_API void RemoveItemFromTree(FSceneOutlinerTreeItemRef InItem);

	/** Called when a child has been removed from the specified parent. Will potentially remove the parent from the tree */
	UE_API void OnChildRemovedFromParent(ISceneOutlinerTreeItem& Parent);

	/** Called when a child has been moved in the tree hierarchy */
	UE_API void OnItemMoved(const FSceneOutlinerTreeItemRef& Item);

	public:
		// Test the filters using stack-allocated data to prevent unnecessary heap allocations
		template <typename TreeItemType, typename TreeItemData>
		FSceneOutlinerTreeItemPtr CreateItemFor(const TreeItemData& Data, TFunctionRef<void(const TreeItemType&)> OnItemPassesFilters, bool bForce = false)
		{
			const TreeItemType Temporary(Data);
			bool bPassesFilters = Filters->PassesAllFilters(Temporary);

			if(FilterCollection)
			{
				bPassesFilters &= FilterCollection->PassesAllFilters(Temporary);
			}
			
			if (bPassesFilters)
			{
				OnItemPassesFilters(Temporary);
			}

			if (SearchBoxFilter)
			{
				bPassesFilters &= SearchBoxFilter->PassesFilter(Temporary);
			}
			
		if (bForce || bPassesFilters)
		{
			FSceneOutlinerTreeItemPtr Result = MakeShareable(new TreeItemType(Data));
			Result->WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(AsShared());
			Result->Flags.bIsFilteredOut = !bPassesFilters;
			Result->Flags.bInteractive = Filters->GetInteractiveState(*Result) && InteractiveFilters->GetInteractiveState(*Result);
			return Result;
		}

		return nullptr;
	}

	/** Instruct the outliner to perform an action on the specified item when it is created */
	UE_API void OnItemAdded(const FSceneOutlinerTreeItemID& ItemID, uint8 ActionMask);

	/** Get the columns to be displayed in this outliner */
	const TMap<FName, TSharedPtr<ISceneOutlinerColumn>>& GetColumns() const
	{
		return Columns;
	}

	UE_API void GetSortedColumnIDs(TArray<FName>& OutColumnIDs) const;

	bool PassesFilters(const ISceneOutlinerTreeItem& Item) const
	{
		return Filters->PassesAllFilters(Item);
	}

	/** @return	Returns true if the text filter is currently active */
	UE_API bool IsTextFilterActive() const;

	bool PassesTextFilter(const FSceneOutlinerTreeItemPtr& Item) const
	{
		return PassesAllFilters(Item);
	}

	bool PassesAllFilters(const FSceneOutlinerTreeItemPtr& Item) const
	{
		bool bPassesFilters = false;
		
		if (SearchBoxFilter)
		{
			bPassesFilters = SearchBoxFilter->PassesFilter(*Item);
		}

		if(FilterCollection)
		{
			bPassesFilters &= FilterCollection->PassesAllFilters(*Item);
		}

		return bPassesFilters;
		
	}

	bool HasSelectorFocus(FSceneOutlinerTreeItemPtr Item) const
	{
		return OutlinerTreeView->Private_HasSelectorFocus(Item);
	}

	/** Handler for when a property changes on any item. Called by the mode */
	UE_API void OnItemLabelChanged(FSceneOutlinerTreeItemPtr ChangedItem, bool bFlashHighlight = true);
	
private:
	/** BindCommands for this Outliner with additional commands to append */
	UE_API void BindCommands(const TSharedPtr<FUICommandList>& InCommandListToAppend);

	/** Map of columns that are shown on this outliner. */
	TMap<FName, TSharedPtr<ISceneOutlinerColumn>> Columns;

	/** Set up the columns required for this outliner */
	UE_API void SetupColumns();

	UE_API void HandleHiddenColumnsChanged();

	/** Refresh the scene outliner columns */
	UE_API void RefreshColumns();

	UE_API void AddColumn_Internal(const FName& ColumnId, const FSceneOutlinerColumnInfo& ColumnInfo, const TMap<FName, bool>& ColumnVisibilities, int32 InsertPosition = INDEX_NONE);
	UE_API void RemoveColumn_Internal(const FName& ColumnId);
	
	/** Populates OutSearchStrings with the strings associated with TreeItem that should be used in searching */
	UE_API void PopulateSearchStrings( const ISceneOutlinerTreeItem& TreeItem, OUT TArray< FString >& OutSearchStrings ) const;

	/** Filter Bar related private functionality */
	
	/** Creates a TextFilter for ISceneOutlinerTreeItem used to save searches as Text Filters */
	UE_API TSharedPtr< SceneOutliner::TreeItemTextFilter > CreateTextFilter() const;

	UE_API bool CompareItemWithClassName(SceneOutliner::FilterBarType InItem, const TSet<FTopLevelAssetPath>&) const;

	/** Delegate for when a filter in the filter bar is changed */
	UE_API void OnFilterBarFilterChanged();

	/** Create the filter bar for this outliner using the specified init options */
	UE_API void CreateFilterBar(const FSceneOutlinerFilterBarOptions& FilterBarOptions);
	
public:
	/** Miscellaneous helper functions */

	/** Scroll the specified item into view */
	UE_API void ScrollItemIntoView(const FSceneOutlinerTreeItemPtr& Item);

	UE_API virtual void SetItemExpansion(const FSceneOutlinerTreeItemPtr& Item, bool bIsExpanded) override;
		
	UE_API bool IsItemExpanded(const FSceneOutlinerTreeItemPtr& Item) const;

	UE_API void ExpandAll();
	UE_API void CollapseAll();

private:

	/** Check whether we should be showing folders or not in this scene outliner */
	UE_API bool ShouldShowFolders() const;

	/** Get an array of selected folders */
	UE_API void GetSelectedFolders(TArray<FFolderTreeItem*>& OutFolders) const;

private:
	/** Tree view event bindings */

	/** Called by STreeView to generate a table row for the specified item */
	UE_API TSharedRef< ITableRow > OnGenerateRowForOutlinerTree( FSceneOutlinerTreeItemPtr Item, const TSharedRef< STableViewBase >& OwnerTable );

	/** Called by STreeView to generate a table row for the specified item is pinned*/
	UE_API TSharedRef< ITableRow > OnGeneratePinnedRowForOutlinerTree(FSceneOutlinerTreeItemPtr Item, const TSharedRef< STableViewBase >& OwnerTable);

	/** Called by STreeView to get child items for the specified parent item */
	UE_API void OnGetChildrenForOutlinerTree( FSceneOutlinerTreeItemPtr InParent, TArray< FSceneOutlinerTreeItemPtr >& OutChildren );

	/** Called by STreeView when the tree's selection has changed */
	UE_API void OnOutlinerTreeSelectionChanged( FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type SelectInfo );

	/** Called by STreeView when the user double-clicks on an item in the tree */
	UE_API void OnOutlinerTreeDoubleClick( FSceneOutlinerTreeItemPtr TreeItem );

	/** Called by STreeView when the user single-clicks on an item in the tree */
	UE_API void OnOutlinerTreeSingleClick( FSceneOutlinerTreeItemPtr TreeItem ) const;

	/** Called by STreeView when an item is scrolled into view */
	UE_API void OnOutlinerTreeItemScrolledIntoView( FSceneOutlinerTreeItemPtr TreeItem, const TSharedPtr<ITableRow>& Widget );

	/** Called when an item in the tree has been collapsed or expanded */
	UE_API void OnItemExpansionChanged(FSceneOutlinerTreeItemPtr TreeItem, bool bIsExpanded) const;

private:
	/** Level, editor and other global event hooks required to keep the outliner up to date */

	UE_API void OnHierarchyChangedEvent(FSceneOutlinerHierarchyChangedData Event);

	/** Handler for when an asset is reloaded */
	UE_API void OnAssetReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

public:
	/** Copy specified folders to clipboard, keeping current clipboard contents if they differ from previous clipboard contents (meaning items were copied) */
	UE_API void CopyFoldersToClipboard(const TArray<FName>& InFolders, const FString& InPrevClipboardContents);

	/** Called by copy and duplicate */
	UE_API void CopyFoldersBegin();

	/** Called by copy and duplicate */
	UE_API void CopyFoldersEnd();

	/** Called by paste and duplicate */
	UE_API void PasteFoldersBegin(TArray<FName> InFolders);

	/** Paste folders end logic */
	UE_API void PasteFoldersEnd();

	/** Called by cut and delete */
	UE_API void DeleteFoldersBegin();

	/** Called by cut and delete */
	UE_API void DeleteFoldersEnd();

	/** Get an array of folders to paste */
	UE_API TArray<FName> GetClipboardPasteFolders() const;

	/** Construct folders export string to be used in clipboard */
	UE_API FString ExportFolderList(TArray<FName> InFolders) const;

	/** Construct array of folders to be created based on input clipboard string */
	UE_API TArray<FName> ImportFolderList(const FString& InStrBuffer) const;

public:
	/** Duplicates current folder and all descendants */
	UE_API void DuplicateFoldersHierarchy();

	FName GetPrimaryColumnName() const;

private:
	/** Cache selected folders during edit delete */
	TArray<FFolderTreeItem*> CacheFoldersDelete;

	/** Cache folders for cut/copy/paste/duplicate */
	TArray<FName> CacheFoldersEdit;

	/** CacheFoldersEdit target root object */
	FFolder::FRootObject CacheFoldersEditRootObject;

	/** Cache folders mapping (old to new) for cut/copy/paste/duplicate */
	TMap<FName, FName> CacheFolderMap;

	/** Cache clipboard contents for cut/copy */
	FString CacheClipboardContents;

	/** Maps pre-existing children during paste or duplicate */
	TMap<FFolder, TArray<FSceneOutlinerTreeItemID>> CachePasteFolderExistingChildrenMap;

	/** Cache which columns are hidden to know what changed */
	TSet<FName> CacheHiddenColumns;

private:
	UE_API bool GetCommonRootObjectFromSelection(FFolder::FRootObject& OutCommonRootObject) const;

	/** Miscellaneous bindings required by the UI */

	/** Called by the editable text control when the filter text is changed by the user */
	UE_API void OnFilterTextChanged( const FText& InFilterText );

	/** Called by the editable text control when a user presses enter or commits their text change */
	UE_API void OnFilterTextCommitted( const FText& InFilterText, ETextCommit::Type CommitInfo );

	/** Called by the filter button to get the image to display in the button */
	UE_API const FSlateBrush* GetFilterButtonGlyph() const;

	/** @return	The filter button tool-tip text */
	UE_API FString GetFilterButtonToolTip() const;

	/** @return	Returns whether the filter status line should be drawn */
	UE_API EVisibility GetFilterStatusVisibility() const;

	/** @return	Returns the filter status text */
	UE_API FText GetFilterStatusText() const;

	/** @return Returns color for the filter status text message, based on success of search filter */
	UE_API FSlateColor GetFilterStatusTextColor() const;

	/**	Returns the current visibility of the Empty label */
	UE_API EVisibility GetEmptyLabelVisibility() const;

	/** @return the selection mode; disabled entirely if in PIE/SIE mode */
	UE_API ESelectionMode::Type GetSelectionMode() const;

	/** @return the content for the view button */
	UE_API TSharedRef<SWidget> GetViewButtonContent(bool bShowFilters);

	/** @return the foreground color for the view button */
	UE_API FSlateColor GetViewButtonForegroundColor() const;

public:

	/** Open a context menu for this scene outliner */
	UE_API TSharedPtr<SWidget> OnOpenContextMenu();

	UE_API void FillFoldersSubMenu(UToolMenu* Menu) const;
	UE_API void AddMoveToFolderOutliner(UToolMenu* Menu) const;
	UE_API void FillSelectionSubMenu(UToolMenu* Menun) const;
	UE_API TSharedRef<TSet<FFolder>> GatherInvalidMoveToDestinations() const;

	/** Called to select descendants of the currently selected folders */
	UE_API void SelectFoldersDescendants(bool bSelectImmediateChildrenOnly = false);

	/** Moves the current selection to the specified folder path */
	UE_API void MoveSelectionTo(const FFolder& NewParent);

	/** Create a new folder under the specified parent name (NAME_None for root) */
	UE_API void CreateFolder();

private:
	/** Called when the user has clicked the button to add a new folder */
	UE_API FReply OnCreateFolderClicked();

private:

	/** Context menu opening delegate provided by the client */
	FOnContextMenuOpening OnContextMenuOpening;

	TSharedPtr<FSharedSceneOutlinerData> SharedData;

	/** List of pending operations to be applied to the tree */
	TArray<SceneOutliner::FPendingTreeOperation> PendingOperations;

	/** Map of actions to apply to new tree items */
	TMap<FSceneOutlinerTreeItemID, uint8> NewItemActions;

	/** Our tree view */
	TSharedPtr< SSceneOutlinerTreeView > OutlinerTreeView;

	/** A map of all items we have in the tree */
	FSceneOutlinerTreeItemMap TreeItemMap;

	/** Pending tree items that are yet to be added the tree */
	FSceneOutlinerTreeItemMap PendingTreeItemMap;

	/** Pending tree items that are yet to be removed from the tree */
	FSceneOutlinerTreeItemMap PendingTreeItemMap_Removal;

	/** Folders pending selection */
	TArray<FFolder> PendingFoldersSelect;

	/** Root level tree items */
	TArray<FSceneOutlinerTreeItemPtr> RootTreeItems;

	/** The button that displays view options */
	TSharedPtr<SComboButton> ViewOptionsComboButton;

	/** Command list to be processed during the OnKeyDown, extended by derived Mode through BindCommands */
	TSharedPtr<FUICommandList> CommandList;

private:

	/** Called when SceneOutlinerModule column permission list changes. */
	UE_API void OnColumnPermissionListChanged();

	/** Structure containing information relating to the expansion state of parent items in the tree */
	typedef TMap<FSceneOutlinerTreeItemID, bool> FParentsExpansionState;
	
	/** Cached expansion state info */
	mutable FParentsExpansionState CachedExpansionStateInfo;

private:

	/** True if the outliner needs to be repopulated at the next appropriate opportunity, usually because our
		item set has changed in some way. */
	uint8 bNeedsRefresh : 1;

	/** Processing a full refresh until pending items are processed */
	uint8 bProcessingFullRefresh : 1;

	/** true if the Scene Outliner should do a full refresh. */
	uint8 bFullRefresh : 1;

	/** true if the Scene Outliner should refresh selection */
	uint8 bSelectionDirty : 1;

	/** True if the Scene Outliner is currently responding to a level visibility change */
	uint8 bDisableIntermediateSorting : 1;

	uint8 bNeedsColumRefresh : 1;

	/** True if the outliner should cache changes to column visibility into the config */
	uint8 bShouldCacheColumnVisibility : 1;

	/** True if we are forcing the underlying tree view to automatically expand all parents when searching */
	mutable uint8 bForceParentItemsExpanded : 1;

	/** Reentrancy guard */
	bool bIsReentrant;

	/** The delay (in seconds) before a UI sort and refresh is executed after being requested. 0 means the next refresh will be immediate */
	float UIRefreshDelay = 0.0f;

	/* Widget containing the filtering text box */
	TSharedPtr< SFilterSearchBox > FilterTextBoxWidget;

	/** The header row of the scene outliner */
	TSharedPtr< SHeaderRow > HeaderRowWidget;

	/** A collection of filters used to filter the displayed items and folders in the scene outliner */
	TSharedPtr< FSceneOutlinerFilters > Filters;

	/** A collection of extra filters applied on top of existing Filters to determine if items are interactive or not in the scene outliner */
	TSharedPtr< FSceneOutlinerFilters > InteractiveFilters;

	/** The TextFilter attached to the SearchBox widget of the Scene Outliner */
	TSharedPtr< SceneOutliner::TreeItemTextFilter > SearchBoxFilter;

	/** The FilterBar attached to this outliner to filter down assets further */
	TSharedPtr< SFilterBar< SceneOutliner::FilterBarType > > FilterBar;

	/** The FilterCollection belonging to the Filter Bar */
	TSharedPtr< TFilterCollection< SceneOutliner::FilterBarType > > FilterCollection;

	/** True if the search box will take keyboard focus next frame */
	bool bPendingFocusNextFrame;

	/** The tree item that is currently pending a rename */
	TWeakPtr<ISceneOutlinerTreeItem> PendingRenameItem;

	TMap<FName, const FSlateBrush*> CachedIcons;

	SceneOutliner::FTreeItemPtrEvent OnDoubleClickOnTreeEvent;

	SceneOutliner::FOnItemSelectionChanged OnItemSelectionChanged;

	FSimpleMulticastDelegate OnFullRefreshDone;
	
	/** The primary column for this outliner, used to determine initial sorting and where the hierarchy arrows appear */
	FName PrimaryColumnName;
private:

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override {};
	virtual FString GetReferencerName() const override
	{
		return TEXT("SceneOutliner::SSceneOutliner");
	}

	/** Functions relating to sorting */

	/** Timer for PIE/SIE mode to sort the outliner. */
	float SortOutlinerTimer;

	/** true if the outliner currently needs to be sorted */
	bool bSortDirty;

	/** Specify which column to sort with */
	FName SortByColumn;

	/** Currently selected sorting mode */
	EColumnSortMode::Type SortMode;

	/** Identifier for this outliner (Set through FSceneOutlinerInitializationOptions) */
	FName OutlinerIdentifier;

	/** true if the hierarchy of items is pinned at the top of the outliner */
	bool bShouldStackHierarchyHeaders = true;

	UE_API void ToggleStackHierarchyHeaders();

	UE_API bool ShouldStackHierarchyHeaders() const;

	/** Handles column sorting mode change */
	UE_API void OnColumnSortModeChanged( const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode );

	/** Sort the specified array of items based on the current sort column */
	UE_API void SortItems(TArray<FSceneOutlinerTreeItemPtr>& Items) const;

	UE_API virtual uint32 GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const override;

	/** Handler for recursively expanding/collapsing items */
	UE_API void SetItemExpansionRecursive(FSceneOutlinerTreeItemPtr Model, bool bInExpansionState);

	/**
	 * Get a mutable version of the outliner config for setting values.
	 * @returns		The outliner config for this outliner.
	 * @note		If FSceneOutlinerInitializationOptions.OutlinerIdentifier is not set, it is not possible to store settings for this outliner.
	 */
	UE_API struct FSceneOutlinerConfig* GetMutableConfig();

	/**
	 * Get a const version of the outliner config for getting values.
	 * @returns		The outliner config for this outliner.
	 * @note		If FSceneOutlinerInitializationOptions.OutlinerIdentifier is not set, it is not possible to retrieve settings for this outliner.
	 */
	UE_API const FSceneOutlinerConfig* GetConstConfig() const;

	UE_API void SaveConfig();

	TSharedPtr<FSceneOutlinerSCCHandler> SourceControlHandler;

public:
	UE_API virtual TSharedPtr<FSceneOutlinerTreeItemSCC> GetItemSourceControl(const FSceneOutlinerTreeItemPtr& InItem) override;

	UE_API void AddSourceControlMenuOptions(UToolMenu* Menu);

	UE_API void FilterByType(TSharedPtr<FCustomClassFilterData> CustomClassFilterData);
	UE_API void FilterByTypeCategory(TSharedPtr<FFilterCategory> TypeCategory, TArray<TSharedPtr<FCustomClassFilterData>> Classes);
	UE_API bool NeedsRefresh() const;

	/**
	 * Add a type filter to the scene outliner's filter bar. Added separately so it can appear in the 'Type Filters' section
	 * of the Add Filter menu rather than "Other Filters". The filter will appear under its primary category (set in the Filter)
	 * and each additional category in the "Add Filter" menu. Toggling from any category activates the same filter instance,
	 * providing a shared state.
	 */
	UE_API void AddTypeFilterToFilterBar(const TSharedRef<FFilterBase<SceneOutliner::FilterBarType>>& InFilter,
		TArrayView<const TSharedPtr<FFilterCategory>> InAdditionalCategories = {}) const;

	UE_API void CreateCustomTextFilter(const FCustomTextFilterData& InFilterData, bool bApplyFilter);
	UE_API void ModifyCustomTextFilter(const FCustomTextFilterData& InFilterData, const TSharedPtr<ICustomTextFilter<SceneOutliner::FilterBarType>>& InFilter);
	UE_API void DeleteCustomTextFilter(const TSharedPtr<ICustomTextFilter<SceneOutliner::FilterBarType>>& InFilter);
};

struct FSceneOutlinerMenuHelper
{
	static UE_API void AddMenuEntryCreateFolder(FToolMenuSection& InSection, SSceneOutliner& InOutliner);
	static UE_API void AddMenuEntryCleanupFolders(FToolMenuSection& InSection, ULevel* InLevel);
};

#undef UE_API
