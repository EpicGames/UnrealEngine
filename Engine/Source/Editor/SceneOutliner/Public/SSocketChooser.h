// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"
#include "Layout/SlateRect.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"
#include "Misc/TextFilterExpressionEvaluator.h"

class UStaticMesh;

class SSocketChooserPopup : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnSocketChosen, FName );

	/** Info about one socket */
	struct FSocketInfo
	{
		FComponentSocketDescription Description;

		/** Cached filter context for faster comparison */
		FBasicStringFilterExpressionContext FilterContext;

		static TSharedRef<FSocketInfo> Make(FComponentSocketDescription Description)
		{
			return MakeShareable( new FSocketInfo(Description) );
		}

	protected:
		FSocketInfo(FComponentSocketDescription InDescription)
			: Description(InDescription)
			, FilterContext(InDescription.Name.ToString())
		{}
	};


	/** The Component that contains the sockets we are choosing from */
	TWeakObjectPtr<USceneComponent> SceneComponent;

	/** StaticMesh that we want to pick a socket for. Will only be used if no SkeletalMesh */
	TWeakObjectPtr<UStaticMesh> StaticMesh;

	/** Array of shared pointers to socket infos */
	TArray< TSharedPtr<FSocketInfo> > Sockets;

	/** Filtered list of sockets */
	TArray< TSharedPtr<FSocketInfo> > FilteredSockets;

	/** Delegate to call when OK button is pressed */
	FOnSocketChosen OnSocketChosen;

	/** The combo box */
	TSharedPtr< SListView< TSharedPtr<FSocketInfo> > > SocketListView;

	/** Compiled filter search terms. */
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilterPtr;

	/** Search box widget */
	TSharedPtr<SWidget> SearchBox;

	SLATE_BEGIN_ARGS( SSocketChooserPopup )
		: _SceneComponent(NULL)
		, _ProvideNoSocketOption(true)
		{}

		/** A component that contains sockets */
		SLATE_ARGUMENT( USceneComponent*, SceneComponent )

		/** Called when the text is chosen. */
		SLATE_EVENT( FOnSocketChosen, OnSocketChosen )

		/** Control if the 'none' socket is shown */
		SLATE_ARGUMENT(bool, ProvideNoSocketOption)
	SLATE_END_ARGS()

private:
	/** Cache a weak pointer to my window */
	TWeakPtr<SWindow> WidgetWindow;

public:
	/** Called to create a widget for each socket */
	TSharedRef<ITableRow> MakeItemWidget( TSharedPtr<FSocketInfo> SocketInfo, const TSharedRef<STableViewBase>& OwnerTable )
	{
		const FSlateBrush* Brush = FAppStyle::GetBrush(TEXT("SocketIcon.None"));
		
		if (SocketInfo->Description.Type == EComponentSocketType::Socket)
		{
			Brush = FAppStyle::GetBrush( TEXT("SocketIcon.Socket") );
		}
		else if (SocketInfo->Description.Type == EComponentSocketType::Bone) 
		{
			Brush = FAppStyle::GetBrush(TEXT("SocketIcon.Bone"));
		}
		
		return	
			SNew( STableRow< TSharedPtr<FSocketInfo> >, OwnerTable )
			. Content()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(Brush)
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromName(SocketInfo->Description.Name))
					.HighlightText_Lambda([this]() { return TextFilterPtr->GetFilterText(); })
				]
			];
	}

	/** Called when item is selected */
	void SelectedSocket(TSharedPtr<FSocketInfo> SocketInfo, ESelectInfo::Type SelectType)
	{
		if(SelectType == ESelectInfo::OnMouseClick)
		{
			ChooseSocket(SocketInfo);
		}
	}

	/** Select an item */
	void ChooseSocket(TSharedPtr<FSocketInfo> SocketInfo)
	{
		const FName SocketName = SocketInfo->Description.Name;

		FSlateApplication::Get().DismissAllMenus();

		if(OnSocketChosen.IsBound())
		{
			OnSocketChosen.Execute(SocketName);
		}
	}

	void Construct( const FArguments& InArgs )
	{
		OnSocketChosen = InArgs._OnSocketChosen;
		SceneComponent = InArgs._SceneComponent;

		// Add in 'No Socket' selection if required
		if (InArgs._ProvideNoSocketOption)
		{
			Sockets.Add( SSocketChooserPopup::FSocketInfo::Make(FComponentSocketDescription(NAME_None, EComponentSocketType::Invalid)) );
		}

		TextFilterPtr = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString));

		// Build set of sockets
		if (SceneComponent != NULL)
		{
			TArray<FComponentSocketDescription> Descriptions;
			SceneComponent->QuerySupportedSockets(/*out*/ Descriptions);
			for (auto DescriptionIt = Descriptions.CreateConstIterator(); DescriptionIt; ++DescriptionIt)
			{
				Sockets.Add( SSocketChooserPopup::FSocketInfo::Make(*DescriptionIt) );
			}

			FilteredSockets.Append(Sockets);
		}

		// Then make widget
		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Background")))
			.Padding(5.f)
			.Content()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 1.0f)
				[
					SNew(STextBlock)
					.Font( FAppStyle::GetFontStyle(TEXT("SocketChooser.TitleFont")) )
					.Text( NSLOCTEXT("SocketChooser", "ChooseSocketOrBoneLabel", "Choose Socket or Bone") )
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 1.0f)
				[
					SAssignNew(SearchBox, SSearchBox)
					.OnTextChanged(this, &SSocketChooserPopup::HandleSearchTextChanged)
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.MaxHeight(512.f)
				[
					SNew(SBox)
					.WidthOverride(256.f)
					.Content()
					[
						SAssignNew(SocketListView, SListView< TSharedPtr<FSocketInfo> >)
						.ListItemsSource( &FilteredSockets)
						.OnGenerateRow( this, &SSocketChooserPopup::MakeItemWidget )
						.OnSelectionChanged( this, &SSocketChooserPopup::SelectedSocket )
						.SelectionMode(ESelectionMode::Single)
					]
				]
			]
		];
	}
	
	/** SWidget interface */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{		
		// Make sure that my window is on-screen
		TSharedPtr<SWindow> Window = CheckAndGetWindowPtr();
		if(Window.IsValid())
		{
			FVector2f WindowLocation = Window->IsMorphing() ? Window->GetMorphTargetPosition() : Window->GetPositionInScreen();
			FVector2f WindowSize = Window->GetDesiredSize();
			FSlateRect Anchor(WindowLocation.X, WindowLocation.Y, WindowLocation.X, WindowLocation.Y);
			WindowLocation = FSlateApplication::Get().CalculatePopupWindowPosition( Anchor, WindowSize, false );

			// Update the window's position!
			if( Window->IsMorphing() )
			{
				if( WindowLocation != Window->GetMorphTargetPosition() )
				{
					Window->UpdateMorphTargetShape( FSlateRect(WindowLocation.X, WindowLocation.Y, WindowLocation.X + WindowSize.X, WindowLocation.Y + WindowSize.Y) );
				}
			}
			else
			{
				if( WindowLocation != Window->GetPositionInScreen() )
				{
					Window->MoveWindowTo(WindowLocation);
				}
			}
		}
	}

	void HandleSearchTextChanged(const FText& InText)
	{
		TextFilterPtr->SetFilterText(InText);

		FilteredSockets.Reset();

		if (InText.IsEmpty())
		{
			FilteredSockets.Append(Sockets);
		}
		else
		{
			for (TSharedPtr<FSocketInfo>& SocketInfo : Sockets)
			{
				if (TextFilterPtr->TestTextFilter(SocketInfo->FilterContext))
				{
					FilteredSockets.Add(SocketInfo);
				}
			}
		}

		SocketListView->RequestListRefresh();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if(InKeyEvent.GetKey() == EKeys::Enter)
		{
			TArray<TSharedPtr<FSocketInfo>> SelectedItems = SocketListView->GetSelectedItems();
            if(SelectedItems.Num() > 0)
            {
            	ChooseSocket(SelectedItems[0]);
            	return FReply::Handled();
            }
		}

		return FReply::Unhandled();
	}

	virtual bool SupportsKeyboardFocus() const override
	{
		return SearchBox->SupportsKeyboardFocus();
	}

	virtual bool HasKeyboardFocus() const override
	{
		// Since keyboard focus is forwarded to our editable text, we will test it instead
		return SearchBox->HasKeyboardFocus();
	}

	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override
	{
		// Forward keyboard focus to our editable text widget
		return FReply::Handled().SetUserFocus(SearchBox.ToSharedRef(), InFocusEvent.GetCause());
	}

private:
	/** Check and get the cached window pointer so that we don't need to find it if we already have it */
	TSharedPtr<SWindow> CheckAndGetWindowPtr()
	{
		if(WidgetWindow.IsValid())
		{
			return WidgetWindow.Pin();
		}
		else
		{
			TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());
			WidgetWindow = Window;
			return Window;
		}
	}
};
