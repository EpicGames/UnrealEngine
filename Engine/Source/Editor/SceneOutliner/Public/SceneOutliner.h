// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

#include "SceneOutlinerFwd.h"

#include "ISceneOutliner.h"
#include "ISceneOutlinerColumn.h"

#include "SceneOutlinerPublicTypes.h"

#include "SceneOutlinerModule.h"

#include "ISceneOutlinerTreeItem.h"

#include "ActorTreeItem.h"
#include "FolderTreeItem.h"
#include "WorldTreeItem.h"

#include "SceneOutlinerFilters.h"
