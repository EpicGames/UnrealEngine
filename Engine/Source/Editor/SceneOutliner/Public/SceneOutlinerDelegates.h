// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnrealEdMisc.h"

class FSceneOutlinerDelegates
{
public:
	/** Return a single FSceneOutlinerDelegates object */
	static FSceneOutlinerDelegates& Get()
	{
		// return the singleton object
		static FSceneOutlinerDelegates Singleton;
		return Singleton;
	}

	/**	Broadcasts whenever the current selection changes */
	FSimpleMulticastDelegate SelectionChanged;

	/** Broadcasts whenever a Component Selection Changes */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnComponentSelectionChanged, class UActorComponent*);
	FOnComponentSelectionChanged OnComponentSelectionChanged;

	/** Broadcasts whenever a Component has been modified */
	FSimpleMulticastDelegate OnComponentsUpdated;

};
