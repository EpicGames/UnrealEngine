// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "DragAndDrop/CompositeDragDropOp.h"
#include "HAL/Platform.h"
#include "ISceneOutlinerTreeItem.h"
#include "Input/DragAndDrop.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "SceneOutlinerFwd.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"

#define UE_API SCENEOUTLINER_API

class SWidget;
struct FSlateBrush;
struct ISceneOutlinerTreeItem;

/** Enum to describe the compatibility of a drag drop operation */
enum class ESceneOutlinerDropCompatibility : uint8
{
	Compatible,
	Incompatible,
	MultipleSelection_Incompatible,
	CompatibleAttach,
	IncompatibleGeneric,
	CompatibleGeneric,
	CompatibleMultipleAttach,
	IncompatibleMultipleAttach,
	CompatibleDetach,
	CompatibleMultipleDetach,
};

/** Consolidated drag/drop with parsing functions for the scene outliner */
struct FSceneOutlinerDragDropPayload
{
	/** Default constructor, resulting in unset contents */
	FSceneOutlinerDragDropPayload(const FDragDropOperation& InOperation = FDragDropOperation())
	: SourceOperation(InOperation)
	{
	}

	/** Populate this payload from an array of tree items */
	template<typename TreeType>
	FSceneOutlinerDragDropPayload(const TArray<TreeType>& InDraggedItems, const FDragDropOperation& InOperation = FDragDropOperation())
	: SourceOperation(InOperation)
	{
		for (const auto& Item : InDraggedItems)
		{
			DraggedItems.Add(Item);
		}
	}

	/** Returns true if the payload has an item of a specified type */
	template <typename TreeType>
	bool Has() const
	{
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : DraggedItems)
		{
			if (const auto ItemPtr = Item.Pin())
			{
				if (ItemPtr->IsA<TreeType>())
				{
					return true;
				}
			}
		}
		return false;
	}

	/** Return an array of all tree items in the payload which are of a specified type */
	template <typename TreeType>
	TArray<TreeType*> Get() const
	{
		TArray<TreeType*> Result;
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : DraggedItems)
		{
			if (const auto ItemPtr = Item.Pin())
			{
				if (TreeType* CastedItem = ItemPtr->CastTo<TreeType>())
				{
					Result.Add(CastedItem);
				}
			}
		}
		return Result;
	}

	/** Apply a function to each item in the payload */
	template <typename TreeType>
	void ForEachItem(TFunctionRef<void(TreeType&)> Func) const
	{
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : DraggedItems)
		{
			if (const auto ItemPtr = Item.Pin())
			{
				if (TreeType* CastedItem = ItemPtr->CastTo<TreeType>())
				{
					Func(*CastedItem);
				}
			}
		}
	}
		
	/** Use a selector to retrieve an array of a specific data type from the items in the payload */
	template <typename DataType>
	TArray<DataType> GetData(TFunctionRef<bool(TWeakPtr<ISceneOutlinerTreeItem>, DataType&)> Selector) const
	{
		TArray<DataType> Result;
		for (TWeakPtr<ISceneOutlinerTreeItem>& Item : DraggedItems)
		{
			DataType Data;
			if (Selector(Item, Data))
			{
				Result.Add(Data);
			}
		}
		return Result;
	}

	/** List of all dragged items */
	mutable TArray<TWeakPtr<ISceneOutlinerTreeItem>> DraggedItems;

	/** The source FDragDropOperation */
	const FDragDropOperation& SourceOperation;
};

/** Struct used for validation of a drag/drop operation in the scene outliner */
struct FSceneOutlinerDragValidationInfo
{
	/** The tooltip type to display on the operation */
	ESceneOutlinerDropCompatibility CompatibilityType;

	/** The tooltip text to display on the operation */
	FText ValidationText;

	/** Construct this validation information out of a tootip type and some text */
	FSceneOutlinerDragValidationInfo(const ESceneOutlinerDropCompatibility InCompatibilityType, const FText InValidationText)
		: CompatibilityType(InCompatibilityType)
		, ValidationText(InValidationText)
	{}

	/** Return a generic invalid result */
	static FSceneOutlinerDragValidationInfo Invalid()
	{
		return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
	}
		
	/** @return true if this operation is valid, false otheriwse */ 
	bool IsValid() const
	{
		switch(CompatibilityType)
		{
		case ESceneOutlinerDropCompatibility::Compatible:
		case ESceneOutlinerDropCompatibility::CompatibleAttach:
		case ESceneOutlinerDropCompatibility::CompatibleGeneric:
		case ESceneOutlinerDropCompatibility::CompatibleMultipleAttach:
		case ESceneOutlinerDropCompatibility::CompatibleDetach:
		case ESceneOutlinerDropCompatibility::CompatibleMultipleDetach:
			return true;
		default:
			return false;
		}
	}
};

/** A drag/drop operation that was started from the scene outliner */
struct FSceneOutlinerDragDropOp : public FCompositeDragDropOp
{
	DRAG_DROP_OPERATOR_TYPE(FSceneOutlinerDragDropOp, FCompositeDragDropOp);
		
	UE_API FSceneOutlinerDragDropOp();

	using FDragDropOperation::Construct;

	void ResetTooltip()
	{
		OverrideText = FText();
		OverrideIcon = nullptr;
	}

	void SetTooltip(FText InOverrideText, const FSlateBrush* InOverrideIcon)
	{
		OverrideText = InOverrideText;
		OverrideIcon = InOverrideIcon;
	}

	UE_API virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

private:

	UE_API EVisibility GetOverrideVisibility() const;
	UE_API EVisibility GetDefaultVisibility() const;

	FText OverrideText;
	FText GetOverrideText() const { return OverrideText; }

	const FSlateBrush* OverrideIcon;
	const FSlateBrush* GetOverrideIcon() const { return OverrideIcon; }
};

#undef UE_API
