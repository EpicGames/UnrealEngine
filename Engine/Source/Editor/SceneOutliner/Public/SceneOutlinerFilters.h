// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "ISceneOutlinerTreeItem.h"
#include "Internationalization/Text.h"
#include "Misc/FilterCollection.h"
#include "Misc/IFilter.h"
#include "SceneOutlinerFwd.h"
#include "Templates/SharedPointer.h"

#define UE_API SCENEOUTLINER_API

class FMenuBuilder;
class FSceneOutlinerFilter;
struct FSceneOutlinerFilters;

/**
	* Contains information used to create a filter which will be displayed as user toggleable filter
	*/
class FSceneOutlinerFilterInfo
{ 
public:
	FSceneOutlinerFilterInfo(const FText& InFilterTitle, const FText& InFilterTooltip, bool bInActive, const FCreateSceneOutlinerFilter& InFactory = FCreateSceneOutlinerFilter())
		: FilterTitle(InFilterTitle)
		, FilterTooltip(InFilterTooltip)
		, bActive(bInActive)
		, Factory(InFactory)
	{}

	/** Initialize and apply a new filter */
	UE_API void InitFilter(TSharedPtr<FSceneOutlinerFilters> InFilters);

	/** Add menu for this filter */
	UE_API void AddMenu(FMenuBuilder& InMenuBuilder);

	UE_API bool IsFilterActive() const;

	DECLARE_EVENT_OneParam(FSceneOutlinerFilterInfo, FOnToggle, bool);
	FOnToggle& OnToggle() { return OnToggleEvent; }

private:
	UE_API void ApplyFilter(bool bActive);
	UE_API void ToggleFilterActive();

	TWeakPtr<FSceneOutlinerFilters> Filters;

	TSharedPtr<FSceneOutlinerFilter> Filter;

	FText FilterTitle;
	FText FilterTooltip;
	bool bActive;

	FOnToggle OnToggleEvent;

	FCreateSceneOutlinerFilter Factory;
};

/** A filter that can be applied to any type in the tree */
class FSceneOutlinerFilter : public IFilter<const ISceneOutlinerTreeItem&>
{
public:
	/** Enum to specify how items that are not explicitly handled by this filter should be managed */
	enum class EDefaultBehaviour : uint8 { Pass, Fail };

	/** Event that is fired if this filter changes */
	DECLARE_DERIVED_EVENT(FSceneOutlinerFilter, IFilter<const ISceneOutlinerTreeItem&>::FChangedEvent, FChangedEvent);
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }

protected:

	/**	The event that broadcasts whenever a change occurs to the filter */
	FChangedEvent ChangedEvent;

	/** Default result of the filter when not overridden in derived classes */
	const EDefaultBehaviour DefaultBehaviour;

	/** Constructor to specify the default result of a filter */
	FSceneOutlinerFilter(EDefaultBehaviour InDefaultBehaviour)
		: DefaultBehaviour(InDefaultBehaviour)
	{}

private:
	/** Transient result from the filter operation. Only valid until the next invocation of the filter. */
	mutable bool bTransientFilterResult;

	/** Check whether the specified item passes our filter */
	virtual bool PassesFilter(const ISceneOutlinerTreeItem& InItem) const override
	{
		bTransientFilterResult = (DefaultBehaviour == EDefaultBehaviour::Pass);
		return bTransientFilterResult;
	}
public:
	/** 
		* Check if an item should be interactive according to this filter.
		* Default behavior just checks if it passes the filter or not.
		*/
	virtual bool GetInteractiveState(const ISceneOutlinerTreeItem& InItem) const
	{
		return PassesFilter(InItem);
	}
};

/** Outliner filter which will be applied on items which match the specified type */
template <class T>
struct TSceneOutlinerFilter : public FSceneOutlinerFilter
{
	TSceneOutlinerFilter(EDefaultBehaviour InDefaultBehaviour)
		: FSceneOutlinerFilter(InDefaultBehaviour)
	{}

	virtual bool PassesFilter(const ISceneOutlinerTreeItem& InItem) const override
	{
		if (const T* CastedItem = InItem.CastTo<T>())
		{
			return PassesFilterImpl(*CastedItem);
		}
		return DefaultBehaviour == EDefaultBehaviour::Pass;
	}

	virtual bool PassesFilterImpl(const T& InItem) const
	{
		return DefaultBehaviour == EDefaultBehaviour::Pass;
	}

	virtual bool GetInteractiveState(const ISceneOutlinerTreeItem& InItem) const
	{
		if (const T* CastedItem = InItem.CastTo<T>())
		{
			return GetInteractiveStateImpl(*CastedItem);
		}
		return DefaultBehaviour == EDefaultBehaviour::Pass;
	}

	// If not overriden will just default to testing against PassesFilter
	virtual bool GetInteractiveStateImpl(const T& InItem) const
	{
		return PassesFilterImpl(InItem);
	}
};


/** Predicate based filter for the outliner */
template <class T>
struct TSceneOutlinerPredicateFilter : public TSceneOutlinerFilter<T>
{
	using TFilterPredicate = typename T::FFilterPredicate;
	using TInteractivePredicate = typename T::FInteractivePredicate;

	/** Predicate used to filter tree items */
	mutable TFilterPredicate FilterPred;
	mutable TInteractivePredicate InteractivePred;

	TSceneOutlinerPredicateFilter(TFilterPredicate InFilterPred, FSceneOutlinerFilter::EDefaultBehaviour InDefaultBehaviour, TInteractivePredicate InInteractivePredicate = TInteractivePredicate())
		: TSceneOutlinerFilter<T>(InDefaultBehaviour)
		, FilterPred(InFilterPred)
		, InteractivePred(InInteractivePredicate)
	{}

	virtual bool PassesFilterImpl(const T& InItem) const override
	{
		return InItem.Filter(FilterPred);
	}

	virtual bool GetInteractiveStateImpl(const T& InItem) const override 
	{
		if (InteractivePred.IsBound())
		{
			return InItem.GetInteractiveState(InteractivePred);
		}

		// If not interactive state impl is provided, default to interactive if filter passes
		return PassesFilterImpl(InItem);
	}
};

/** Scene outliner filters class. This class wraps a collection of filters and allows items of any type to be tested against the entire set. */
struct FSceneOutlinerFilters : public TFilterCollection<const ISceneOutlinerTreeItem&>
{
	/** Overridden to ensure we only ever have FSceneOutlinerFilters added */
	int32 Add(const TSharedPtr<FSceneOutlinerFilter>& Filter)
	{
		return TFilterCollection::Add(Filter);
	}

	/** Test whether this tree item passes all filters, and set its interactive state according to the filter it failed (if applicable) */
	bool GetInteractiveState(const ISceneOutlinerTreeItem& InItem) const
	{
		for (const auto& Filter : ChildFilters)
		{
			if (!StaticCastSharedPtr<FSceneOutlinerFilter>(Filter)->GetInteractiveState(InItem))
			{
				return false;
			}
		}

		return true;
	}

	/** Add a filter predicate to this filter collection */
	template <typename T>
	void AddFilterPredicate(typename T::FFilterPredicate InFilterPred, FSceneOutlinerFilter::EDefaultBehaviour InDefaultBehaviour = FSceneOutlinerFilter::EDefaultBehaviour::Pass, typename T::FInteractivePredicate InInteractivePred = typename T::FInteractivePredicate())
	{
		Add(MakeShareable(new TSceneOutlinerPredicateFilter<T>(InFilterPred, InDefaultBehaviour, InInteractivePred)));
	}
};

#undef UE_API
