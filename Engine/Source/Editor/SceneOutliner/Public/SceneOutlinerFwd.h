// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreDelegates.h"

class ISceneOutliner;
class ISceneOutlinerColumn;

struct FSceneOutlinerTreeItemID;

struct FSceneOutlinerInitializationOptions;
struct FSharedSceneOutlinerData;

struct ISceneOutlinerTreeItem;
struct FActorTreeItem;
struct FWorldTreeItem;
struct FFolderTreeItem;
struct FComponentTreeItem;

typedef TSharedPtr<ISceneOutlinerTreeItem> FSceneOutlinerTreeItemPtr;
typedef TSharedRef<ISceneOutlinerTreeItem> FSceneOutlinerTreeItemRef;

typedef TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr> FSceneOutlinerTreeItemMap;

class ISceneOutlinerHierarchy;
class ISceneOutlinerMode;

class SSceneOutliner;

class FSceneOutlinerFilter;
struct FSceneOutlinerFilters;

struct FSceneOutlinerDragDropPayload;
struct FSceneOutlinerDragValidationInfo;

DECLARE_DELEGATE_OneParam( FOnSceneOutlinerItemPicked, TSharedRef<ISceneOutlinerTreeItem> );

DECLARE_DELEGATE_OneParam( FCustomSceneOutlinerDeleteDelegate, const TArray<TWeakPtr<ISceneOutlinerTreeItem>>&  )

/** A delegate used to factory a new column type */
DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<ISceneOutlinerColumn>, FCreateSceneOutlinerColumn, ISceneOutliner& );

/** A delegate used to factory a new filter type */
DECLARE_DELEGATE_RetVal( TSharedRef<FSceneOutlinerFilter>, FCreateSceneOutlinerFilter );

/** A delegate used to factory a new scene outliner using the given init options */
DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<ISceneOutliner>, FSceneOutlinerFactory, FSceneOutlinerInitializationOptions);


class FSceneOutlinerTreeItemSCC;
