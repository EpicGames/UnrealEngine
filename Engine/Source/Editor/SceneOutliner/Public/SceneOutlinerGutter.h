// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Images/SImage.h"
#include "SceneOutlinerPublicTypes.h"
#include "ScopedTransaction.h"
#include "ISceneOutlinerColumn.h"
#include "ISceneOutlinerTreeItem.h"

#define UE_API SCENEOUTLINER_API

class ISceneOutliner;
template<typename ItemType> class STableRow;

/**
 * A gutter for the SceneOutliner which handles setting and visualizing item visibility
 */
class FSceneOutlinerGutter : public ISceneOutlinerColumn
{
public:

	/**	Constructor */
	UE_API FSceneOutlinerGutter(ISceneOutliner& Outliner);

	virtual ~FSceneOutlinerGutter() = default;

	static FName GetID() { return FSceneOutlinerBuiltInColumnTypes::Gutter(); }

	static FText GetDisplayName() { return FSceneOutlinerBuiltInColumnTypes::Gutter_Localized(); }

	// -----------------------------------------
	// ISceneOutlinerColumn Implementation
	UE_API virtual FName GetColumnID() override;

	UE_API virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;

	UE_API virtual const TSharedRef< SWidget > ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;

	UE_API virtual void Tick(double InCurrentTime, float InDeltaTime) override;

	virtual bool SupportsSorting() const override { return true; }

	UE_API virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const override;
	// -----------------------------------------

	/** Check whether the specified item is visible */
	FORCEINLINE bool IsItemVisible(const ISceneOutlinerTreeItem& Item)
	{
		return VisibilityCache.GetVisibility(Item);
	}

protected:

	/** Weak pointer back to the scene outliner - required for setting visibility on current selection. */
	TWeakPtr<ISceneOutliner> WeakOutliner;

	/** Get and cache visibility for items. Cached per-frame to avoid expensive recursion. */
	FSceneOutlinerVisibilityCache VisibilityCache;
};

template<>
struct TWidgetTypeTraits<class SVisibilityWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** Widget responsible for managing the visibility for a single item */
class SVisibilityWidget : public SImage
{
public:
	SLATE_BEGIN_ARGS(SVisibilityWidget) {}
	SLATE_END_ARGS()

		/** Construct this widget */
		UE_API void Construct(const FArguments& InArgs, TWeakPtr<FSceneOutlinerGutter> InWeakColumn, TWeakPtr<ISceneOutliner> InWeakOutliner, TWeakPtr<ISceneOutlinerTreeItem> InWeakTreeItem, const STableRow<FSceneOutlinerTreeItemPtr>* InRow);

protected:

	/** Returns whether the widget is enabled or not */
	virtual bool IsEnabled() const { return true; }

	/** Get the brush for this widget */
	UE_API virtual const FSlateBrush* GetBrush() const;

	/** Start a new drag/drop operation for this widget */
	UE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** If a visibility drag drop operation has entered this widget, set its item to the new visibility state */
	UE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	UE_API FReply HandleClick();

	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	/** Called when the mouse button is pressed down on this widget */
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Process a mouse up message */
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
	UE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

	/** Called when visibility change happens on an Item. */
	UE_API virtual void OnSetItemVisibility(ISceneOutlinerTreeItem& Item, const bool bNewVisibility);

	/** Whether visibility change should propagate down to children. */
	virtual bool ShouldPropagateVisibilityChangeOnChildren() const { return true; }

	UE_API virtual FSlateColor GetForegroundColor() const;

	/** Check if the specified item is visible */
	static UE_API bool IsVisible(const FSceneOutlinerTreeItemPtr& Item, const TSharedPtr<FSceneOutlinerGutter>& Column);

	/** Check if our wrapped tree item is visible */
	UE_API bool IsVisible() const;

	/** Set the item this widget is responsible for to be hidden or shown */
	UE_API void SetIsVisible(const bool bVisible);

	/** The tree item we relate to */
	TWeakPtr<ISceneOutlinerTreeItem> WeakTreeItem;

	/** Reference back to the outliner so we can set visibility of a whole selection */
	TWeakPtr<ISceneOutliner> WeakOutliner;

	/** Weak pointer back to the column */
	TWeakPtr<FSceneOutlinerGutter> WeakColumn;

	/** Weak pointer back to the row */
	const STableRow<FSceneOutlinerTreeItemPtr>* Row;

	/** Scoped undo transaction */
	TUniquePtr<FScopedTransaction> UndoTransaction;

	/** Visibility brushes for the various states */
	const FSlateBrush* VisibleHoveredBrush;
	const FSlateBrush* VisibleNotHoveredBrush;
	const FSlateBrush* NotVisibleHoveredBrush;
	const FSlateBrush* NotVisibleNotHoveredBrush;
};

#undef UE_API
