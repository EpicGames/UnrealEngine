// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Columns/SceneOutlinerColumns.h"
#include "SceneOutlinerPublicTypes.h"

#define UE_API SCENEOUTLINER_API

namespace SceneOutliner
{
	// Class to hold common functionality needed by the Outliner and helpful to modules creating Outliner instances
	class FSceneOutlinerHelpers
	{
	public:
		UE_DEPRECATED(5.5, "FSceneOutlinerHelpers::GetExternalPackageName has been deprecated implement/use ISceneOutlinerTreeItem::GetPackageName instead")
		static UE_API FString GetExternalPackageName(const ISceneOutlinerTreeItem& TreeItem);
		
		UE_DEPRECATED(5.5, "FSceneOutlinerHelpers::GetExternalPackage has been deprecated implement/use ISceneOutlinerTreeItem::GetPackageName instead")
		static UE_API UPackage* GetExternalPackage(const ISceneOutlinerTreeItem& TreeItem);
		
		static UE_API void ChooseRepresentingWorld(TWeakObjectPtr<UWorld>& OutWorld,
			const TWeakObjectPtr<UWorld>& InSpecifiedWorldToDisplay = nullptr,
			const TWeakObjectPtr<UWorld>& InUserChosenWorld = nullptr);
		
		static UE_API TSharedPtr<SWidget> GetClassHyperlink(UObject* InObject);
		
		static UE_API void PopulateExtraSearchStrings(const ISceneOutlinerTreeItem& TreeItem, TArray< FString >& OutSearchStrings);

		static UE_API bool ValidateFolderName(const FFolder& InFolder, UWorld* World, const FText& InLabel, FText& OutErrorMessage);

		static UE_API void RenameFolder(const FFolder& InFolder, const FText& NewFolderName, UWorld* World);
		
		static UE_API bool IsFolderCurrent(const FFolder& InFolder, UWorld* World);
		
		static UE_API TWeakObjectPtr<AActor> GetActorFromOutlinerTreeItem(const ISceneOutlinerTreeItem& TreeItem, const TSharedPtr<ISceneOutliner>& Outliner);
	};
};

#undef UE_API
