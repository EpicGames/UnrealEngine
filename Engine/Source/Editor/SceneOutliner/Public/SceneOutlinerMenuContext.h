// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "SceneOutlinerMenuContext.generated.h"

class SSceneOutliner;

UCLASS(MinimalAPI)
class USceneOutlinerMenuContext : public UObject
{
	GENERATED_BODY()
public:

	USceneOutlinerMenuContext(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
		bShowParentTree = false;
		bRepresentingPartitionedWorld = false;
		bRepresentingGameWorld = false;
		NumSelectedItems = 0;
		NumSelectedFolders = 0;
		NumWorldsSelected = 0;
		NumPinnedItems = 0;
	}

	TWeakPtr<SSceneOutliner> SceneOutliner;

	bool bShowParentTree;
	bool bRepresentingGameWorld;
	bool bRepresentingPartitionedWorld;
	int32 NumSelectedItems;
	int32 NumSelectedFolders;
	int32 NumWorldsSelected;
	int32 NumPinnedItems;
};
