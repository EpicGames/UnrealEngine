// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerPublicTypes.h"
#include "Misc/NamePermissionList.h"

class ICustomSceneOutliner;
class ISceneOutlinerColumn;
struct FSceneOutlinerInitializationOptions;

/** Delegate used with the Scene Outliner in 'actor picking' mode.  You'll bind a delegate when the
	outliner widget is created, which will be fired off when an actor is selected in the list */
DECLARE_DELEGATE_OneParam(FOnActorPicked, AActor*);
/** Delegate used with the Scene Outliner in 'component picking' mode.  You'll bind a delegate when the
	outliner widget is created, which will be fired off when an actor is selected in the list */
DECLARE_DELEGATE_OneParam(FOnComponentPicked, UActorComponent*);

/**
 * Implements the Scene Outliner module.
 */
class FSceneOutlinerModule
	: public IModuleInterface
{
public:

	FSceneOutlinerModule();

	/**
	 * Creates a scene outliner widget
	 *
	 * @param	InitOptions						Programmer-driven configuration for this widget instance
	 * @param	OutlinerModeFactory				Factory delegate used to create the outliner mode
	 *
	 * @return	New scene outliner widget
	 */
	virtual TSharedRef<ISceneOutliner> CreateSceneOutliner(
		const FSceneOutlinerInitializationOptions& InitOptions) const;

	/* Some common scene outliners */

	/** Creates an actor picker widget. Calls the OnActorPickedDelegate when an item is selected. */
	virtual TSharedRef<ISceneOutliner> CreateActorPicker(
		const FSceneOutlinerInitializationOptions& InInitOptions,
		const FOnActorPicked& OnActorPickedDelegate,
		TWeakObjectPtr<UWorld> SpecifiedWorld = nullptr, bool bHideLevelInstanceHierarchy = true) const;

	/** Creates a component picker widget. Calls the OnComponentPickedDelegate when an item is selected. */
	virtual TSharedRef<ISceneOutliner> CreateComponentPicker(
		const FSceneOutlinerInitializationOptions& InInitOptions,
		const FOnComponentPicked& OnComponentPickedDelegate,
		TWeakObjectPtr<UWorld> SpecifiedWorld = nullptr) const;

	/** Creates an actor browser widget (also known as a World Outliner). */
	virtual TSharedRef<ISceneOutliner> CreateActorBrowser(
		const FSceneOutlinerInitializationOptions& InInitOptions,
		TWeakObjectPtr<UWorld> SpecifiedWorld = nullptr) const;


	/** Register a factory to create a custom Scene Outliner */
	UE_INTERNAL SCENEOUTLINER_API virtual void RegisterCustomSceneOutlinerFactory(FName ID, FSceneOutlinerFactory InOutlinerFactory) final;

	/** Unregister a factory to create a custom Scene Outliner */
	UE_INTERNAL SCENEOUTLINER_API virtual void UnregisterCustomSceneOutlinerFactory(FName ID) final;

	/** Try to create a custom scene outliner using a registered factory (nullptr if ID was not registered) */
	UE_INTERNAL SCENEOUTLINER_API virtual TSharedPtr<ISceneOutliner> CreateCustomRegisteredOutliner(FName ID, FSceneOutlinerInitializationOptions InInitOptions) final;

	/** Check if a custom scene outliner factory is registered with the given ID */
	UE_INTERNAL SCENEOUTLINER_API virtual bool IsCustomSceneOutlinerFactoryRegistered(FName ID) final;

	/** Add the columns present in the level editor's Outliner (Actor Browser) to the given init options
	 *  @param InWorld The world the Outliner initialized by InInitOptions will look it, defaults to the level editor's world if nullptr
	 */
	virtual void CreateActorBrowserColumns(FSceneOutlinerInitializationOptions& InInitOptions, UWorld* InWorld = nullptr) const;
	
	/** Column permission list */
	TSharedRef<FNamePermissionList>& GetColumnPermissionList() { return ColumnPermissionList; }

	/** Delegate that broadcasts when column permission list changes. */
	DECLARE_MULTICAST_DELEGATE(FOnColumnPermissionListChanged);
	FOnColumnPermissionListChanged& OnColumnPermissionListChanged() { return ColumnPermissionListChanged; }

	/** Delegate that broadcasts when creating actor browser columns. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCreateActorBrowserColumns, FSceneOutlinerInitializationOptions&, UWorld*);
	FOnCreateActorBrowserColumns& OnCreateActorBrowserColumns() { return CreateActorBrowserColumnsDelegate; }

public:
	/** Register a new type of column available to all scene outliners */
	template< typename T >
	void RegisterColumnType()
	{
		auto ID = T::GetID();
		if ( !ColumnMap.Contains( ID ) )
		{
			auto CreateColumn = []( ISceneOutliner& Outliner ){
				return TSharedRef< ISceneOutlinerColumn >( MakeShareable( new T(Outliner) ) );
			};

			ColumnMap.Add( ID, FCreateSceneOutlinerColumn::CreateStatic( CreateColumn ) );
		}
	}

	/** Register a new type of default column available to all scene outliners */
	template< typename T >
	void RegisterDefaultColumnType(FSceneOutlinerColumnInfo InColumnInfo)
	{
		auto ID = T::GetID();
		if ( !ColumnMap.Contains( ID ) )
		{
			auto CreateColumn = []( ISceneOutliner& Outliner ){
				return TSharedRef< ISceneOutlinerColumn >( MakeShareable( new T(Outliner) ) );
			};

			ColumnMap.Add( ID, FCreateSceneOutlinerColumn::CreateStatic( CreateColumn ) );
			DefaultColumnMap.Add( ID, InColumnInfo);
		}
	}

	/** Unregister a previously registered column type */
	template< typename T >
	void UnRegisterColumnType()
	{
		ColumnMap.Remove( T::GetID() );
		DefaultColumnMap.Remove( T::GetID() );
	}

	/** Factory a new column from the specified name. Returns null if no type has been registered under that name. */
	TSharedPtr< ISceneOutlinerColumn > FactoryColumn( FName ID, ISceneOutliner& Outliner ) const
	{
		if ( auto* Factory = ColumnMap.Find( ID ) )
		{
			return Factory->Execute(Outliner);
		}
		
		return nullptr;
	}

	void CreateActorInfoColumns(FSceneOutlinerInitializationOptions& InInitOptions, UWorld* WorldPtr = nullptr) const;
	
	/** Map of column type name -> default column info */
	TMap< FName, FSceneOutlinerColumnInfo> DefaultColumnMap;

private:

	/** Map of column type name -> factory delegate */
	TMap< FName, FCreateSceneOutlinerColumn > ColumnMap;

	/** Column permission list used to filter scene ouliner columns. */
	TSharedRef<FNamePermissionList> ColumnPermissionList;

	/** Delegate that broadcasts when column permission list changes. */
	FOnColumnPermissionListChanged ColumnPermissionListChanged;

	/** Delegate that broadcasts when creating actor browser columns. */
	FOnCreateActorBrowserColumns CreateActorBrowserColumnsDelegate;
	
	TMap< FName, FSceneOutlinerFactory> CustomOutlinerFactories;

public:

	// IModuleInterface interface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
