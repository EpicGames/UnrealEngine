// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "Framework/SlateDelegates.h"
#include "Layout/Margin.h"
#include "SceneOutlinerFilters.h"

#define UE_API SCENEOUTLINER_API

#define LOCTEXT_NAMESPACE "SceneOutlinerPublicTypes"

class FCustomClassFilterData;
class FExtender;
class FFilterCategory;
struct FToolMenuContext;
class FUICommandList;
template<typename FilterType> class FFilterBase;

DECLARE_DELEGATE_TwoParams(FSceneOutlinerModifyContextMenu, FName& /* MenuName */, FToolMenuContext& /* MenuContext */);

/** A delegate used as a factory to defer mode creation in the outliner */
DECLARE_DELEGATE_RetVal_OneParam(ISceneOutlinerMode*, FCreateSceneOutlinerMode, SSceneOutliner*);

namespace SceneOutliner
{
	/** The type of item that the Outliner's Filter Bar operates on */
	typedef const ISceneOutlinerTreeItem& FilterBarType;
}

/** Container for built in column types. Function-static so they are available without linking */
struct FSceneOutlinerBuiltInColumnTypes
{
	#define DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(ColumnID, ColumnName, ColumnKey, ColumnLocalizedTextLiteral) \
	static FName& ColumnID() \
	{ \
		static FName ColumnID = ColumnName; \
		return ColumnID; \
	} \
	static const FText& ColumnID##_Localized() \
	{ \
		static FText ColumnID##_Localized = LOCTEXT(ColumnKey, ColumnLocalizedTextLiteral); \
		return ColumnID##_Localized; \
	}

	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(Label, "Item Label", "ItemLabelColumnName", "Item Label");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(Gutter, "Visibility", "VisibilityColumnName", "Visibility");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(ActorInfo, "Type", "TypeColumnName", "Type");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(SourceControl, "Revision Control", "SourceControlColumnName", "Revision Control");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(Mobility, "Mobility", "SceneOutlinerMobilityColumn", "Mobility"); 
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(Level, "Level", "SceneOutlinerLevelColumn", "Level");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(Layer, "Layer", "SceneOutlinerLayerColumn", "Layer");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(DataLayer, "Data Layer", "SceneOutlinerDataLayerColumn", "Data Layer");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(ContentBundle, "Content Bundle", "SceneOutlinerContentBundleColumn", "Content Bundle");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(SubPackage, "Sub Package", "SceneOutlinerSubPackageColumn", "Sub Package");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(Pinned, "Pinned", "SceneOutlinerPinnedColumn", "Pinned");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(IDName, "ID Name", "SceneOutlinerIDNameColumn", "ID Name");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(PackageShortName, "Package Short Name", "SceneOutlinerPackageShortNameColumn", "Package Short Name");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(UncachedLights, "Uncached Lights", "SceneOutlinerUncachedLightsColumn", "# Uncached Lights");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(Socket, "Socket", "SceneOutlinerSocketColumn", "Socket");
	DEFINE_SCENEOUTLINER_BUILTIN_COLUMN_TYPE(Unsaved, "Unsaved", "UnsavedColumnName", "Unsaved");
};

/** Visibility enum for scene outliner columns */
enum class ESceneOutlinerColumnVisibility : uint8
{
	/** This column defaults to being visible on the scene outliner */
	Visible,

	/** This column defaults to being invisible, yet still available on the scene outliner */
	Invisible,
};

/** Column information for the scene outliner */
struct FSceneOutlinerColumnInfo
{
	FSceneOutlinerColumnInfo(
		ESceneOutlinerColumnVisibility InVisibility, uint8 InPriorityIndex, const FCreateSceneOutlinerColumn& InFactory = FCreateSceneOutlinerColumn()
		, bool inCanBeHidden = true, TOptional<float> InFillSize = TOptional<float>(), TAttribute<FText> InColumnLabel = TAttribute<FText>()
		, EHeaderComboVisibility InHeaderComboVisibility = EHeaderComboVisibility::OnHover, FOnGetContent InOnGetMenuContent = FOnGetContent())
		: Visibility(InVisibility), PriorityIndex(InPriorityIndex), bCanBeHidden(inCanBeHidden)
		, Factory(InFactory), FillSize(InFillSize), ColumnLabel(InColumnLabel)
		, HeaderComboVisibility(InHeaderComboVisibility), OnGetHeaderContextMenuContent(InOnGetMenuContent)
	{
	}

	FSceneOutlinerColumnInfo() {}

	FSceneOutlinerColumnInfo(const FSceneOutlinerColumnInfo& InColumnInfo)
		: Visibility(InColumnInfo.Visibility), PriorityIndex(InColumnInfo.PriorityIndex)
		, bCanBeHidden(InColumnInfo.bCanBeHidden), Factory(InColumnInfo.Factory)
		, FillSize(InColumnInfo.FillSize), ColumnLabel(InColumnInfo.ColumnLabel)
		, HeaderComboVisibility(InColumnInfo.HeaderComboVisibility)
		, OnGetHeaderContextMenuContent(InColumnInfo.OnGetHeaderContextMenuContent)
	{}

	ESceneOutlinerColumnVisibility 	Visibility;
	uint8				PriorityIndex;
	bool bCanBeHidden;
	FCreateSceneOutlinerColumn	Factory;
	TOptional< float > FillSize;
	TAttribute<FText> ColumnLabel; // Override for the column name used instead of ID if specified (use this if you want the column name to be localizable)

	/** Hides the button on each header cell which normally shows up when OnGetHeaderContextMenuContent is bound */
	EHeaderComboVisibility HeaderComboVisibility;
	FOnGetContent OnGetHeaderContextMenuContent;
};

/** Settings for the scene outliner which can be quieried publicly */
struct FSharedSceneOutlinerData
{
	/**	Invoked whenever the user attempts to delete an actor from within a Scene Outliner in the actor browsing mode */
	FCustomSceneOutlinerDeleteDelegate CustomDelete;

	/** Modify context menu before display */
	FSceneOutlinerModifyContextMenu ModifyContextMenu;

	/** Map of column types available to the scene outliner, along with default ordering */
	TMap<FName, FSceneOutlinerColumnInfo> ColumnMap;
		
	/** Whether the Scene Outliner should display parent actors in a Tree */
	bool bShowParentTree : 1;

	/** True to only show folders in this outliner */
	bool bOnlyShowFolders : 1;

	/** Show transient objects */
	bool bShowTransient : 1;

public:

	/** Constructor */
	FSharedSceneOutlinerData()
		: bShowParentTree( true )
		, bOnlyShowFolders( false )
		, bShowTransient( false )
	{}

	/** Set up a default array of columns for this outliner */
	UE_API void UseDefaultColumns();
};

/* Settings for the Filter Bar attached to the Scene Outliner. Can be specified through FSceneOutlinerInitializationOptions */
struct FSceneOutlinerFilterBarOptions
{
	/** If true, the Scene Outliner has a filter bar attached to it */
	bool bHasFilterBar = false;

	/** These are the custom filters that the Scene Outliner will have. All active filters will be AND'd together to test
	 *  against.
	 *  @see FGenericFilter on how to create generic filters
	 */
	TArray<TSharedRef<FFilterBase<SceneOutliner::FilterBarType>>> CustomFilters;

	/** These are the asset type filters that the Scene Outliner will have. All active filters will be OR'd together to
	 *  test against.
	 *  Can be created using IAssetTypeActions or UClass (@see constructor)
	 */
	TArray<TSharedRef<FCustomClassFilterData>> CustomClassFilters;

	/** If true, share this Outliner filter bar's custom text filters with the level editor outliners */
	bool bUseSharedSettings = false;

	/** The category to expand in the filter menu (There must be at least one filter attached to the category) */
	TSharedPtr<FFilterCategory> CategoryToExpand;
};

/**
	* Settings for the Scene Outliner set by the programmer before spawning an instance of the widget.  This
	* is used to modify the outliner's behavior in various ways, such as filtering in or out specific classes
	* of actors.
	*/
struct FSceneOutlinerInitializationOptions : FSharedSceneOutlinerData
{	
	/** True if we should draw the header row above the tree view */
	bool bShowHeaderRow : 1;
	
	/**
	 * Can select the columns generated by right clicking on the header menu.
	 * FColumn with ShouldGenerateWidget set can not be selected.
	 * FColumn with MenuContent will still be displayed.
	 */
	bool bCanSelectGeneratedColumns : 1;
	
	/** Whether the Scene Outliner should expose its searchbox */
	bool bShowSearchBox : 1;

	/** If true, the search box will gain focus when the scene outliner is created */
	bool bFocusSearchBoxWhenOpened : 1;

	/** If true, the Scene Outliner will expose a Create New Folder button */
	bool bShowCreateNewFolder : 1;

	/** Optional collection of filters to use when filtering in the Scene Outliner */
	TSharedPtr<FSceneOutlinerFilters> Filters;		

	FCreateSceneOutlinerMode ModeFactory;

	/** Identifier for this outliner; NAME_None if this view is anonymous (Needs to be specified to save visibility of columns in EditorConfig)*/
	FName OutlinerIdentifier;

	/** Init options related to the filter bar */
	FSceneOutlinerFilterBarOptions FilterBarOptions;

	/** The primary column for this outliner, used to determine initial sorting and where the hierarchy arrows appear */
	FName PrimaryColumnName;

	/** Additional commands to append to the scene outliner command list */
	TSharedPtr<FUICommandList> CommandList;

	/** Constructor */
	FSceneOutlinerInitializationOptions()
		: bShowHeaderRow( true )
		, bCanSelectGeneratedColumns( true )
		, bShowSearchBox( true )
		, bFocusSearchBoxWhenOpened( false )
		, bShowCreateNewFolder( true )
		, Filters( new FSceneOutlinerFilters )
		, ModeFactory()
		, OutlinerIdentifier(NAME_None)
		, PrimaryColumnName(FSceneOutlinerBuiltInColumnTypes::Label())
	{}
};

/** Default metrics for outliner tree items */
struct FSceneOutlinerDefaultTreeItemMetrics
{
	static int32	RowHeight() { return 20; };
	static int32	IconSize() { return 16; };
	static FMargin	IconPadding() { return FMargin(0.f, 1.f, 6.f, 1.f); };
};

/** A struct which gets, and caches the visibility of a tree item */
struct FSceneOutlinerVisibilityCache
{
	/** Map of tree item to visibility */
	mutable TMap<const ISceneOutlinerTreeItem*, bool> VisibilityInfo;

	/** Get an item's visibility based on its children */
	UE_API bool RecurseChildren(const ISceneOutlinerTreeItem& Item) const;

	UE_API bool GetVisibility(const ISceneOutlinerTreeItem& Item) const;
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
