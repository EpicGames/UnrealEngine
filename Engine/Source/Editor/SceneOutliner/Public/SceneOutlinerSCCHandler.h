// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneOutlinerPublicTypes.h"
#include "ToolMenu.h"
#include "SceneOutlinerTreeItemSCC.h"
#include "UnrealEdMisc.h"

class FSceneOutlinerSCCHandler : public TSharedFromThis<class FSceneOutlinerSCCHandler>
{
public:
	FSceneOutlinerSCCHandler();
	~FSceneOutlinerSCCHandler();

	bool AddSourceControlMenuOptions(UToolMenu* Menu, TArray<FSceneOutlinerTreeItemPtr> InSelectedItems);

	TSharedPtr<FSceneOutlinerTreeItemSCC> GetItemSourceControl(const FSceneOutlinerTreeItemPtr& InItem) const;

private:

	bool AllowExecuteSourceControlRevert() const;
	bool AllowExecuteSourceControlRevertUnsaved() const;
	bool CanExecuteSourceControlActions() const;
	void CacheCanExecuteVars();
	bool CanExecuteSCC() const;
	bool CanExecuteSCCCheckOut() const;
	bool CanExecuteSCCCheckIn() const;
	bool CanExecuteSCCRevert() const;
	bool CanExecuteSCCHistory() const;
	bool CanExecuteSCCRefresh() const;
	bool CanExecuteSCCShowInChangelist() const;
	void FillSourceControlSubMenu(UToolMenu* Menu);
	void GetSelectedPackageNames(TArray<FString>& OutPackageNames) const;
	void GetSelectedPackages(TArray<UPackage*>& OutPackages) const;
	void ExecuteSCCRefresh();
	void ExecuteSCCCheckOut();
	void ExecuteSCCCheckIn();
	void ExecuteSCCRevert();
	void ExecuteSCCHistory();
	void ExecuteSCCShowInChangelist();

	void OnMapChanged(UWorld* InWorld, EMapChangeType MapChangedType);

	TArray<FSceneOutlinerTreeItemPtr> SelectedItems;

	// todo: Sync and Revert functionality is not enabled currently
	// due to needing to unload/reload the actor and having to deal
	// with any potential dependencies. You can read more about this
	// decision here: https://github.com/EpicGames/UnrealEngine/pull/9310#issuecomment-1171505628
	bool bCanExecuteSCC = false;
	bool bCanExecuteSCCCheckOut = false;
	bool bCanExecuteSCCCheckIn = false;
	bool bCanExecuteSCCHistory = false;
	bool bCanExecuteSCCRevert = false;
	bool bUsesSnapshots = false;
	bool bUsesChangelists = false;

	mutable TMap<FSceneOutlinerTreeItemPtr, TSharedPtr<FSceneOutlinerTreeItemSCC>> ItemSourceControls;
};
