// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerPublicTypes.h"
#include "ISceneOutlinerColumn.h"

#define UE_API SCENEOUTLINER_API

template<typename ItemType> class STableRow;

/** A column for the SceneOutliner that displays the SCC Information */
class FSceneOutlinerSourceControlColumn : public ISceneOutlinerColumn
{

public:
	FSceneOutlinerSourceControlColumn(ISceneOutliner& SceneOutliner) : WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared())) {}

	virtual ~FSceneOutlinerSourceControlColumn() = default;

	static FName GetID() { return FSceneOutlinerBuiltInColumnTypes::SourceControl(); }

	static FText GetDisplayName() { return FSceneOutlinerBuiltInColumnTypes::SourceControl_Localized(); }

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation

	UE_API virtual FName GetColumnID() override;

	UE_API virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;

	UE_API virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;

	virtual bool SupportsSorting() const override { return false; }

	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////
private:
	UE_API const FSlateBrush* GetHeaderIconBadge() const;

	TWeakPtr<ISceneOutliner> WeakSceneOutliner;
};

#undef UE_API
