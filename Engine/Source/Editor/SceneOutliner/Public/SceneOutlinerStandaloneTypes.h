// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "UObject/ObjectKey.h"
#include "Templates/MaxSizeof.h"
#include "Folder.h"
#include "SceneOutlinerFwd.h"

#define UE_API SCENEOUTLINER_API

struct FFolderKey
{
	FFolderKey(const FName& InPath, const FFolder::FRootObject& InRootObject)
		: Path(InPath)
		, RootObjectKey(InRootObject)
	{}

	FORCEINLINE bool operator == (const FFolderKey& InOther) const
	{
		return (Path == InOther.Path) && (RootObjectKey == InOther.RootObjectKey);
	}

	FName Path;
	FFolder::FRootObject RootObjectKey;
};

FORCEINLINE uint32 GetTypeHash(FFolderKey Key)
{
	return HashCombineFast(GetTypeHash(Key.Path), GetTypeHash(Key.RootObjectKey));
}

/** Variant type that defines an identifier for a tree item. Assumes 'trivial relocatability' as with many unreal containers. */
struct FSceneOutlinerTreeItemID
{
public:
	using FUniqueID = uint64;

	enum class EType : uint8 { Object, Folder, UniqueID, Guid, Null };

	/** Default constructed null item ID */
	FSceneOutlinerTreeItemID() : Type(EType::Null), CachedHash(0) {}

	/** ID representing a UObject */
	FSceneOutlinerTreeItemID(const UObject* InObject) : Type(EType::Object)
	{
		check(InObject);
		new (Data) FObjectKey(InObject);
		CachedHash = CalculateTypeHash();
	}
	FSceneOutlinerTreeItemID(const FObjectKey& InKey) : Type(EType::Object)
	{
		new (Data) FObjectKey(InKey);
		CachedHash = CalculateTypeHash();
	}

	/** ID representing a folder */
	FSceneOutlinerTreeItemID(const FFolderKey& InKey) : Type(EType::Folder)
	{
		new (Data) FFolderKey(InKey);
		CachedHash = CalculateTypeHash();
	}

	FSceneOutlinerTreeItemID(const FFolder& InFolder)
	: FSceneOutlinerTreeItemID(FFolderKey(InFolder.GetPath(), InFolder.GetRootObject()))
	{}

	/** ID representing a generic tree item */
	FSceneOutlinerTreeItemID(const FUniqueID& CustomID) : Type(EType::UniqueID)
	{
		new (Data) FUniqueID(CustomID);
		CachedHash = CalculateTypeHash();
	}

	explicit FSceneOutlinerTreeItemID(const FGuid& InGuid) : Type(EType::Guid)
	{
		new (Data) FGuid(InGuid);
		CachedHash = CalculateTypeHash();
	}

	/** Copy construction / assignment */
	FSceneOutlinerTreeItemID(const FSceneOutlinerTreeItemID& Other)
	{
		*this = Other;
	}
	FSceneOutlinerTreeItemID& operator=(const FSceneOutlinerTreeItemID& Other)
	{
		Type = Other.Type;
		switch(Type)
		{
			case EType::Object:			new (Data) FObjectKey(Other.GetAsObjectKey());		break;
			case EType::Folder:			new (Data) FFolderKey(Other.GetAsFolderRef());		break;
			case EType::Guid:			new (Data) FGuid(Other.GetAsGuid());				break;
			case EType::UniqueID:		new (Data) FUniqueID(Other.GetAsHash());			break;
			default:																		break;
		}

		CachedHash = CalculateTypeHash();
		return *this;
	}

	/** Move construction / assignment */
	FSceneOutlinerTreeItemID(FSceneOutlinerTreeItemID&& Other)
	{
		*this = MoveTemp(Other);
	}
	FSceneOutlinerTreeItemID& operator=(FSceneOutlinerTreeItemID&& Other)
	{
		Swap(*this, Other);
		return *this;
	}

	~FSceneOutlinerTreeItemID()
	{
		switch(Type)
		{
			case EType::Object:			GetAsObjectKey().~FObjectKey();							break;
			case EType::Folder:			GetAsFolderRef().~FFolderKey();							break;
			case EType::Guid:			GetAsGuid().~FGuid();									break;
			case EType::UniqueID:		/* NOP */												break;
			default:																			break;
		}
	}

	bool operator==(const FSceneOutlinerTreeItemID& Other) const
	{
		return Type == Other.Type && CachedHash == Other.CachedHash && Compare(Other);
	}
	bool operator!=(const FSceneOutlinerTreeItemID& Other) const
	{
		return Type != Other.Type || CachedHash != Other.CachedHash || !Compare(Other);
	}

	uint32 CalculateTypeHash() const
	{
		uint32 Hash = 0;
		switch(Type)
		{
			case EType::Object:			Hash = GetTypeHash(GetAsObjectKey());				break;
			case EType::Folder:			Hash = GetTypeHash(GetAsFolderRef());				break;
			case EType::Guid:			Hash = GetTypeHash(GetAsGuid());					break;
			case EType::UniqueID:		Hash = GetTypeHash(GetAsHash());					break;
			default:																		break;
		}

		return HashCombineFast((uint8)Type, Hash);
	}

	friend uint32 GetTypeHash(const FSceneOutlinerTreeItemID& ItemID)
	{
		return ItemID.CachedHash;
	}

private:

	FObjectKey& 	GetAsObjectKey() const 		{ return *reinterpret_cast<FObjectKey*>(Data); }
	FFolderKey& 	GetAsFolderRef() const		{ return *reinterpret_cast<FFolderKey*>(Data); }
	FGuid&			GetAsGuid() const			{ return *reinterpret_cast<FGuid*>(Data); }
	FUniqueID&		GetAsHash() const			{ return *reinterpret_cast<FUniqueID*>(Data); }

	/** Compares the specified ID with this one - assumes matching types */
	bool Compare(const FSceneOutlinerTreeItemID& Other) const
	{
		switch(Type)
		{
			case EType::Object:			return GetAsObjectKey() == Other.GetAsObjectKey();
			case EType::Folder:			return GetAsFolderRef() == Other.GetAsFolderRef();
			case EType::Guid:			return GetAsGuid() == Other.GetAsGuid();
			case EType::UniqueID:		return GetAsHash() == Other.GetAsHash();
			case EType::Null:			return true;
			default: check(false);		return false;
		}
	}

	EType Type;

	uint32 CachedHash;
	static const uint32 MaxSize = TMaxSizeof<FObjectKey, FFolderKey, FGuid, FUniqueID>::Value;
	alignas(8) mutable uint8 Data[MaxSize];
};
	
struct FSceneOutlinerTreeItemType
{
public:
	explicit FSceneOutlinerTreeItemType(const FSceneOutlinerTreeItemType* Parent = nullptr) : ID(++NextUniqueID), ParentType(Parent) {}

	bool operator==(const FSceneOutlinerTreeItemType& Other) const
	{
		return ID == Other.ID || (ParentType != nullptr && *ParentType == Other);
	}

	bool IsA(const FSceneOutlinerTreeItemType& Other) const
	{
		return (ID == Other.ID) || (ParentType && ParentType->IsA(Other));
	}
	
	friend uint32 GetTypeHash(const FSceneOutlinerTreeItemType& Type)
	{
		return Type.ID; 
	}

private:
	static UE_API uint32 NextUniqueID;
	uint32 ID;
	const FSceneOutlinerTreeItemType* ParentType;
};

struct FSceneOutlinerCommonLabelData
{
	TWeakPtr<ISceneOutliner> WeakSceneOutliner;
	static UE_API const FLinearColor DarkColor;

	static UE_API TOptional<FLinearColor> GetForegroundColor(TWeakPtr<ISceneOutliner> WeakSceneOutliner, const ISceneOutlinerTreeItem& TreeItem);

	UE_API TOptional<FLinearColor> GetForegroundColor(const ISceneOutlinerTreeItem& TreeItem) const;

	UE_API bool CanExecuteRenameRequest(const ISceneOutlinerTreeItem& Item) const;
};

/**
	* Contains hierarchy change data.
	* When an item is added, it will contain a pointer to the new item itself.
	* When an item is removed or moved, it will contain the unique ItemID to that item.
	* In the case that a folder is being moved, it will also contain the new path to that folder.
	*/
struct FSceneOutlinerHierarchyChangedData
{
	enum
	{
		Added,
		Removed,
		Moved,
		FolderMoved,
		FullRefresh
	} Type;

	// This event may pass one of two kinds of data, depending on the type of event
	TArray<FSceneOutlinerTreeItemPtr> Items;

	TArray<FSceneOutlinerTreeItemID> ItemIDs;
	// Used for FolderMoved events
	TArray<FFolder> NewPaths;
	/** Actions to apply to items */
	uint8 ItemActions = 0;
};

#undef UE_API
