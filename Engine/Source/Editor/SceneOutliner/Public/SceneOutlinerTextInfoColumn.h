// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "SceneOutlinerPublicTypes.h"
#include "ISceneOutlinerColumn.h"
#include "Misc/CoreDelegates.h"

#define UE_API SCENEOUTLINER_API

class ISceneOutliner;

/** A delegate used to factory a new TextInfo Column */
DECLARE_DELEGATE_RetVal_OneParam(FString, FGetTextForItem, const ISceneOutlinerTreeItem&);


/**
* A custom column for the SceneOutliner which is capable of displaying any text based information related to items
* Note: Only has support for text, any other requirements should have a custom column made
*/
class FTextInfoColumn : public ISceneOutlinerColumn
{

	public:

		/**
		 *	Constructor
		 */
		UE_API FTextInfoColumn(ISceneOutliner& Outliner, const FName InColumnName, const FGetTextForItem& InGetTextForItem, const FText InColumnToolTip);

		virtual ~FTextInfoColumn() = default;

		// Factory function to create a TextInfoColumn (This column is not registered by default because of it's requirements and therefore requires a factory!)
		static UE_API TSharedRef<ISceneOutlinerColumn> CreateTextInfoColumn(ISceneOutliner& Outliner, const FName InColumnName, const FGetTextForItem InGetTextForItem, const FText InColumnToolTip);

		//////////////////////////////////////////////////////////////////////////
		// Begin ISceneOutlinerColumn Implementation

		UE_API virtual FName GetColumnID() override;

		UE_API virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;

		UE_API virtual const TSharedRef< SWidget > ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;

		UE_API virtual void PopulateSearchStrings(const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings) const override;

		UE_API virtual bool SupportsSorting() const override;

		UE_API virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const override;

		// End ISceneOutlinerColumn Implementation
		//////////////////////////////////////////////////////////////////////////

	private:

		UE_API FText GetInfoForItem(TWeakPtr<ISceneOutlinerTreeItem> TreeItem) const;

		FName ColumnName;

		FText ColumnToolTip;

		/** Weak reference to the outliner widget that owns our list */
		TWeakPtr< ISceneOutliner > SceneOutlinerWeak;

		/* Delegate Called to Get Display Text for an item in the column*/
		FGetTextForItem GetTextForItem;
};

#undef UE_API
