// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneOutlinerPublicTypes.h"
#include "SourceControlHelpers.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerTreeItemSCC"

class FUncontrolledChangelistState;

DECLARE_DELEGATE_OneParam(FSourceControlStateChangedDelegate, FSourceControlStatePtr);
DECLARE_DELEGATE_OneParam(FUncontrolledStateChangedDelegate, TSharedPtr<FUncontrolledChangelistState>);


class FSceneOutlinerTreeItemSCC : public TSharedFromThis<FSceneOutlinerTreeItemSCC>
{
public:
	FSceneOutlinerTreeItemSCC(FSceneOutlinerTreeItemPtr InTreeItemPtr);
	~FSceneOutlinerTreeItemSCC();

	// Actually attempt to connect to Revision Control and get the item's status
	void Initialize();

	FSourceControlStatePtr GetSourceControlState();
	FSourceControlStatePtr RefreshSourceControlState();
	TWeakPtr<FUncontrolledChangelistState> GetUncontrolledChangelistState() { return UncontrolledChangelistState; }

	bool HasValidPackage() const { return !ExternalPackageName.IsEmpty(); }
	FString GetPackageName() const { return ExternalPackageName; }
	FString GetPackageFileName() const { return ExternalPackageFileName; }
	UPackage* GetPackage() const;
	UPackage* LoadPackage() const;

	FSourceControlStateChangedDelegate OnSourceControlStateChanged;
	FUncontrolledStateChangedDelegate OnUncontrolledChangelistsStateChanged;
private:
	void ConnectSourceControl();
	void DisconnectSourceControl();
	void HandleSourceControlStateChanged(EStateCacheUsage::Type CacheUsage);
	void HandleSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider);
	void BroadcastNewState(FSourceControlStatePtr SourceControlState);
	void HandleUncontrolledChangelistsStateChanged();

	/** The tree item we relate to */
	FSceneOutlinerTreeItemPtr TreeItemPtr;

	/** Cache the items external package name and filename */
	FString ExternalPackageName;
	FString ExternalPackageFileName;

	/** Source control state changed delegate handle */
	FDelegateHandle SourceControlStateChangedDelegateHandle;

	/** Source control provider changed delegate handle */
	FDelegateHandle SourceControlProviderChangedDelegateHandle;

	/** Actor packaging mode changed delegate handle */
	FDelegateHandle ActorPackingModeChangedDelegateHandle;

	/** Uncontrolled Changelist Changed changed delegate handle */
	FDelegateHandle UncontrolledChangelistChangedHandle;

	/** The uncontrolled changelist this file belongs to, empty if this item is not uncontrolled */
	TSharedPtr<FUncontrolledChangelistState> UncontrolledChangelistState;
};

#undef LOCTEXT_NAMESPACE
