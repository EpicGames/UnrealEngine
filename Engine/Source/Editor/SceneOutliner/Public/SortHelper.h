// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SHeaderRow.h"
#include "ISceneOutlinerTreeItem.h"
#include "Misc/ComparisonUtility.h"

namespace SceneOutliner
{
	/** Wrapper type that will sort FString's using a more natural comparison (correctly sorts numbers and ignores underscores) */
	struct FNumericStringWrapper
	{
		FString String;

		FNumericStringWrapper()
		{}
		FNumericStringWrapper(FString&& InString)
			: String(InString)
		{}
		FNumericStringWrapper(FNumericStringWrapper&& Other)
			: String(MoveTemp(Other.String))
		{}
		FNumericStringWrapper& operator=(FNumericStringWrapper&& Other)
		{
			String = MoveTemp(Other.String);
			return *this;
		}
		FORCEINLINE bool operator<(const FNumericStringWrapper& Other) const
		{
			return UE::ComparisonUtility::CompareNaturalOrder(String, Other.String) < 0;
		}
		FORCEINLINE bool operator>(const FNumericStringWrapper& Other) const
		{
			return UE::ComparisonUtility::CompareNaturalOrder(String, Other.String) > 0;
		}
		FORCEINLINE bool operator==(const FNumericStringWrapper& Other) const
		{
			return String == Other.String;
		}
	private:
		FNumericStringWrapper(const FNumericStringWrapper&);
		FNumericStringWrapper& operator=(const FNumericStringWrapper&);
	};
}


/**
	* Templated helper to alleviate performance problems with sorting based on complex predicates.
	* Example Usage:
	* 		FSceneOutlinerSortHelper<FString>().Primary([](const ISceneOutlinerTreeItem& Item){ return Item->GetString(); }).SortItems(Array);
	*
	* Or:
	* 		FSceneOutlinerSortHelper<FString, FString>()
	*			.Primary(FGetPrimaryString())
	*			.Secondary(FGetSecondaryString())
	*			.SortItems(Array);
	*/
template<typename PrimaryKeyType, typename SecondaryKeyType = int32>
struct FSceneOutlinerSortHelper
{
	typedef TFunction<PrimaryKeyType(const ISceneOutlinerTreeItem&)> FPrimaryFunction;
	typedef TFunction<SecondaryKeyType(const ISceneOutlinerTreeItem&)> FSecondaryFunction;

	FSceneOutlinerSortHelper()
		: PrimarySortMode(EColumnSortMode::None), SecondarySortMode(EColumnSortMode::None)
	{}

	/** Sort primarily by the specified function and mode. Beware the function is a reference, so must be valid for the lifetime of this instance. */
	FSceneOutlinerSortHelper& Primary(FPrimaryFunction&& Function, EColumnSortMode::Type SortMode)
	{
		PrimarySortMode = SortMode;
		PrimaryFunction = MoveTemp(Function);
		return *this;
	}

	/** Sort secondarily by the specified function and mode. Beware the function is a reference, so must be valid for the lifetime of this instance. */
	FSceneOutlinerSortHelper& Secondary(FSecondaryFunction&& Function, EColumnSortMode::Type SortMode)
	{
		SecondarySortMode = SortMode;
		SecondaryFunction = MoveTemp(Function);
		return *this;
	}

	/** Sort the specified array using the current sort settings */
	void Sort(TArray<FSceneOutlinerTreeItemPtr>& Array)
	{
		TArray<FSortPayload> SortData;
		SortData.Reserve(Array.Num());
		const auto End = Array.Num();
		for (int32 Index = 0; Index < End; ++Index)
		{
			const auto& Element = Array[Index];

			PrimaryKeyType PrimaryKey = PrimaryFunction(*Element);

			SecondaryKeyType SecondaryKey;
			if (SecondarySortMode != EColumnSortMode::None)
			{
				SecondaryKey = SecondaryFunction(*Element);
			}

			const int32 DefaultSortID = Element.IsValid() ? GetTypeHash(Element->GetID()) : 0;
			SortData.Emplace(Index, MoveTemp(PrimaryKey), MoveTemp(SecondaryKey), DefaultSortID);
		}

		SortData.Sort([&](const FSortPayload& One, const FSortPayload& Two){
			if (PrimarySortMode == EColumnSortMode::Ascending && One.PrimaryKey != Two.PrimaryKey)
				return One.PrimaryKey < Two.PrimaryKey;
			else if (PrimarySortMode == EColumnSortMode::Descending && One.PrimaryKey != Two.PrimaryKey)
				return One.PrimaryKey > Two.PrimaryKey;

			if (SecondarySortMode == EColumnSortMode::Ascending && One.SecondaryKey != Two.SecondaryKey)
				return One.SecondaryKey < Two.SecondaryKey;
			else if (SecondarySortMode == EColumnSortMode::Descending && One.SecondaryKey != Two.SecondaryKey)
				return One.SecondaryKey > Two.SecondaryKey;

			return One.DefaultSortID < Two.DefaultSortID;
		});

		TArray<FSceneOutlinerTreeItemPtr> NewArray;
		NewArray.Reserve(Array.Num());

		for (const auto& Element : SortData)
		{
			NewArray.Add(Array[Element.OriginalIndex]);
		}
		Array = MoveTemp(NewArray);
	}

private:

	EColumnSortMode::Type 	PrimarySortMode;
	EColumnSortMode::Type 	SecondarySortMode;

	FPrimaryFunction	PrimaryFunction;
	FSecondaryFunction	SecondaryFunction;

	/** Aggregated data from the sort methods. We extract the sort data from all elements first, then sort based on the extracted data. */
	struct FSortPayload
	{
		int32 OriginalIndex;
		
		PrimaryKeyType PrimaryKey;
		SecondaryKeyType SecondaryKey;
		
		int32 DefaultSortID;

		FSortPayload(int32 InOriginalIndex, PrimaryKeyType&& InPrimaryKey, SecondaryKeyType&& InSecondaryKey, int32 InDefaultSortID)
			: OriginalIndex(InOriginalIndex), PrimaryKey(MoveTemp(InPrimaryKey)), SecondaryKey(MoveTemp(InSecondaryKey)), DefaultSortID(InDefaultSortID)
		{}

		FSortPayload(FSortPayload&& Other)
		{
			(*this) = MoveTemp(Other);
		}
		FSortPayload& operator=(FSortPayload&& rhs)
		{
			OriginalIndex = rhs.OriginalIndex;
			PrimaryKey = MoveTemp(rhs.PrimaryKey);
			SecondaryKey = MoveTemp(rhs.SecondaryKey);
			DefaultSortID = rhs.DefaultSortID;
			return *this;
		}

	private:
		FSortPayload(const FSortPayload&);
		FSortPayload& operator=(const FSortPayload&);
	};
};
