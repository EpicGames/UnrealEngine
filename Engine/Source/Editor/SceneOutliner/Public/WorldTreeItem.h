// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "ISceneOutlinerTreeItem.h"
#include "UObject/ObjectKey.h"

#define UE_API SCENEOUTLINER_API

class UToolMenu;

namespace SceneOutliner
{
	/** Get a description of a world to display in the scene outliner */
	FText SCENEOUTLINER_API GetWorldDescription(UWorld* World);
}

/** A tree item that represents an entire world */
struct FWorldTreeItem : ISceneOutlinerTreeItem
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const UWorld*);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const UWorld*);

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(World.Get());
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(World.Get());
	}

	/** The world this tree item is associated with. */
	mutable TWeakObjectPtr<UWorld> World;
		
	/** Constant identifier for this tree item */
	const FObjectKey ID;

	/** Static type identifier for this tree item class */
	static UE_API const FSceneOutlinerTreeItemType Type;

	/** Construct this item from a world */
	UE_API FWorldTreeItem(UWorld* InWorld);
	UE_API FWorldTreeItem(TWeakObjectPtr<UWorld> InWorld);

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return World.IsValid(); }
	UE_API virtual FSceneOutlinerTreeItemID GetID() const override;
	UE_API virtual FString GetDisplayString() const override;
	UE_API virtual bool CanInteract() const override;
	UE_API virtual void GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner) override;
	UE_API virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	/* End ISceneOutlinerTreeItem Implementation */

	/** Open the world settings for the contained world */
	UE_API void OpenWorldSettings() const;
	/** Browse to world asset in content browser */
	UE_API void BrowseToAsset() const;
	UE_API bool CanBrowseToAsset() const;

	/** Get just the name of the world, for tooltip use */
	UE_API FString GetWorldName() const;

	using FContextMenuProviderFunc = TFunction<void(const UWorld*, UToolMenu*)>;
	static UE_API void RegisterContextMenuProvider(FName InProviderName, const FContextMenuProviderFunc& InProviderFunc);
	static UE_API void UnregisterContextMenuProvider(FName InProviderName);

private:
	/** Static map that registers all context menu providers */
	static UE_API TMap<FName, FContextMenuProviderFunc> ContextMenuProviders;
};

#undef UE_API
