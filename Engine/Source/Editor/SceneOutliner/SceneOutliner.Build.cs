// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SceneOutliner : ModuleRules
{
	public SceneOutliner(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"ApplicationCore",
				"InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"ToolMenus",
				"SourceControl",
				"EditorConfig",
				"SourceControlWindows",
				"UncontrolledChangelists",
				"EditorWidgets",
				"ToolWidgets",
				"UnsavedAssetsTracker",
				"AssetDefinition",
				"TypedElementFramework",
				"TypedElementRuntime",
			}
		);
		
	}
}
